{
	"FileVersion": 3,

	"FriendlyName": "Online Services OSS Adapter",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Online Services adapter for Online Subsystem implementations.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "OnlineServicesOSSAdapter",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineServices",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		}
	]
}