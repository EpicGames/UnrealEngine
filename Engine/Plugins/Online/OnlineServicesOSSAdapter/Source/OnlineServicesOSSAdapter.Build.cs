// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineServicesOSSAdapter : ModuleRules
{
	public OnlineServicesOSSAdapter(ReadOnlyTargetRules Target) : base(Target)
    {
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"OnlineServicesInterface",
				"OnlineServicesCommon",
				"OnlineSubsystem",
				"Json"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreOnline",
			}
		);
	}
}
