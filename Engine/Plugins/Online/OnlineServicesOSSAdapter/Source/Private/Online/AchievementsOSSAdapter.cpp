// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/AchievementsOSSAdapter.h"

#include "Interfaces/OnlineAchievementsInterface.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/DelegateAdapter.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/OnlineServicesOSSAdapter.h"

#include "OnlineSubsystem.h"

namespace UE::Online {

void FAchievementsOSSAdapter::Initialize()
{
	Super::Initialize();
}

void FAchievementsOSSAdapter::PostInitialize()
{
	Super::PostInitialize();

	IOnlineSubsystem& SubsystemV1 = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem();

	AchievementsInterface = SubsystemV1.GetAchievementsInterface();
	check(AchievementsInterface);

	ExternalUIInterface = SubsystemV1.GetExternalUIInterface();

	AchievementsInterface->AddOnAchievementUnlockedDelegate_Handle(FOnAchievementUnlockedDelegate::CreateLambda([this](const FUniqueNetId& LocalUserIdV1, const FString& AchievementId)
	{
		const FAccountId LocalUserIdV2 = Services.Get<FAuthOSSAdapter>()->GetAccountId(LocalUserIdV1.AsShared());

		FAchievementStateUpdated AchievementStateUpdated;
		AchievementStateUpdated.LocalAccountId = LocalUserIdV2;
		AchievementStateUpdated.AchievementIds = { AchievementId };
		OnAchievementStateUpdatedEvent.Broadcast(MoveTemp(AchievementStateUpdated));
	}));
}

void FAchievementsOSSAdapter::PreShutdown()
{
	AchievementsInterface->ClearOnAchievementUnlockedDelegates(this);

	Super::PreShutdown();
}

TOnlineAsyncOpHandle<FQueryAchievementDefinitions> FAchievementsOSSAdapter::QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryAchievementDefinitions> Op = GetOp<FQueryAchievementDefinitions>(MoveTemp(Params));

	/**
	 * Only way to get the achi id's is to call GetCachedAchievements, to get an array of achi objects for this player. Hopefully thats the full set!
	 * So we first cache the player achi's with QueryAchievements
	 */
	Op->Then([this](TOnlineAsyncOp<FQueryAchievementDefinitions>& Op)
	{
		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			Promise.SetValue();
			return Future;
		}

		if (AchievementDefinitions.IsSet())
		{
			Op.SetResult({});
			Promise.SetValue();
			return Future;
		}

		AchievementsInterface->QueryAchievements(*LocalUserId, *MakeDelegateAdapter(this, [this, Promise = MoveTemp(Promise), WeakOp = Op.AsWeak()](const FUniqueNetId& LocalUserId, const bool bWasSuccessful) mutable
		{
			if (!bWasSuccessful)
			{
				if (TOnlineAsyncOpPtr<FQueryAchievementDefinitions> Op = WeakOp.Pin())
				{
					Op->SetError(Errors::Unknown());
				}
			}
			Promise.SetValue();
		}));

		return Future;
	})
	.Then([this](TOnlineAsyncOp<FQueryAchievementDefinitions>& Op)
	{
		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return Future;
		}

		AchievementsInterface->QueryAchievementDescriptions(*LocalUserId, *MakeDelegateAdapter(this, [this, Promise = MoveTemp(Promise), WeakOp = Op.AsWeak()](const FUniqueNetId& LocalUserId, const bool bWasSuccessful) mutable
		{
			if (!bWasSuccessful)
			{
				if (TOnlineAsyncOpPtr<FQueryAchievementDefinitions> Op = WeakOp.Pin())
				{
					Op->SetError(Errors::Unknown());
				}
			}
			Promise.SetValue();
		}));

		return Future;
	})
	.Then([this](TOnlineAsyncOp<FQueryAchievementDefinitions>& Op)
	{
		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return;
		}

		TArray<FOnlineAchievement> Achievements;
		EOnlineCachedResult::Type Result = AchievementsInterface->GetCachedAchievements(*LocalUserId, Achievements);
		if (Result != EOnlineCachedResult::Success)
		{
			Op.SetError(Errors::Unknown());
			return;
		}

		FAchievementDefinitionMap Definitions;
		
		for (FOnlineAchievement& Achievement : Achievements)
		{
			FOnlineAchievementDesc AchievementDesc;
			Result = AchievementsInterface->GetCachedAchievementDescription(Achievement.Id, AchievementDesc);
			if (Result != EOnlineCachedResult::Success)
			{
				Op.SetError(Errors::Unknown());
				return;
			}

			FAchievementDefinition Definition;
			Definition.AchievementId = Achievement.Id;
			Definition.UnlockedDisplayName = AchievementDesc.Title;
			Definition.UnlockedDescription = AchievementDesc.UnlockedDesc;
			Definition.LockedDisplayName = AchievementDesc.Title;
			Definition.LockedDescription = AchievementDesc.LockedDesc;
			Definition.bIsHidden = AchievementDesc.bIsHidden;
			
			Definitions.Emplace(MoveTemp(Achievement.Id), MoveTemp(Definition));
		}

		AchievementDefinitions = MoveTemp(Definitions);
		Op.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineResult<FGetAchievementIds> FAchievementsOSSAdapter::GetAchievementIds(FGetAchievementIds::Params&& Params)
{
	const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
	if (!LocalUserId)
	{
		return TOnlineResult<FGetAchievementIds>(Errors::InvalidUser());
	}

	if (!AchievementDefinitions.IsSet())
	{
		// Call QueryAchievementDefinitions first
		return TOnlineResult<FGetAchievementIds>(Errors::InvalidState());
	}

	FGetAchievementIds::Result Result;
	AchievementDefinitions->GenerateKeyArray(Result.AchievementIds);
	return TOnlineResult<FGetAchievementIds>(MoveTemp(Result));
}

TOnlineResult<FGetAchievementDefinition> FAchievementsOSSAdapter::GetAchievementDefinition(FGetAchievementDefinition::Params&& Params)
{
	const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
	if (!LocalUserId)
	{
		return TOnlineResult<FGetAchievementDefinition>(Errors::InvalidUser());
	}

	if (!AchievementDefinitions.IsSet())
	{
		// Should call QueryAchievementDefinitions first
		return TOnlineResult<FGetAchievementDefinition>(Errors::InvalidState());
	}

	const FAchievementDefinition* AchievementDefinition = AchievementDefinitions->Find(Params.AchievementId);
	if (!AchievementDefinition)
	{
		return TOnlineResult<FGetAchievementDefinition>(Errors::NotFound());
	}

	return TOnlineResult<FGetAchievementDefinition>({ *AchievementDefinition });
}

TOnlineAsyncOpHandle<FQueryAchievementStates> FAchievementsOSSAdapter::QueryAchievementStates(FQueryAchievementStates::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryAchievementStates> Op = GetOp<FQueryAchievementStates>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FQueryAchievementStates>& Op)
	{
		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			Promise.SetValue();
			return Future;
		}

		if (!AchievementDefinitions.IsSet())
		{
			// Call QueryAchievementDefinitions first
			Op.SetError(Errors::InvalidState());
			Promise.SetValue();
			return Future;
		}

		AchievementsInterface->QueryAchievements(*LocalUserId, *MakeDelegateAdapter(this, [this, Promise = MoveTemp(Promise), WeakOp = Op.AsWeak()](const FUniqueNetId& LocalUserId, const bool bWasSuccessful) mutable
		{
			if (!bWasSuccessful)
			{
				if (TOnlineAsyncOpPtr<FQueryAchievementStates> Op = WeakOp.Pin())
				{
					Op->SetError(Errors::Unknown());
				}
			}
			Promise.SetValue();
		}));

		return Future;
	})
	.Then([this](TOnlineAsyncOp<FQueryAchievementStates>& Op)
	{
		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return;
		}

		TArray<FOnlineAchievement> Achievements;
		EOnlineCachedResult::Type Result = AchievementsInterface->GetCachedAchievements(*LocalUserId, Achievements);
		if (Result != EOnlineCachedResult::Success)
		{
			Op.SetError(Errors::Unknown());
			return;
		}

		FAchievementStateMap& LocalUserAchievementStates = AchievementStates.Emplace(Op.GetParams().LocalAccountId);

		for (FOnlineAchievement& Achievement : Achievements)
		{
			FAchievementState& AchievementState = LocalUserAchievementStates.Emplace(Achievement.Id);
			AchievementState.AchievementId = Achievement.Id;
			AchievementState.Progress = Achievement.Progress / 100.f;
			// AchievementState.UnlockTime left unset because this does not exist on per-user state, but on the common description...
		}

		Op.SetResult({});
		OnAchievementStatesQueried(Op.GetParams().LocalAccountId);
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();	
}

TOnlineResult<FGetAchievementState> FAchievementsOSSAdapter::GetAchievementState(FGetAchievementState::Params&& Params) const
{
	const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
	if (!LocalUserId)
	{
		return TOnlineResult<FGetAchievementState>(Errors::InvalidUser());
	}

	const FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Params.LocalAccountId);
	if (!LocalUserAchievementStates)
	{
		// Call QueryAchievementStates first
		return TOnlineResult<FGetAchievementState>(Errors::InvalidState());
	}

	const FAchievementState* AchievementState = LocalUserAchievementStates->Find(Params.AchievementId);
	if (!AchievementState)
	{
		return TOnlineResult<FGetAchievementState>(Errors::NotFound());
	}

	return TOnlineResult<FGetAchievementState>({ *AchievementState });
}

TOnlineResult<FDisplayAchievementUI> FAchievementsOSSAdapter::DisplayAchievementUI(FDisplayAchievementUI::Params&& Params)
{
	if (!ExternalUIInterface)
	{
		return TOnlineResult<FDisplayAchievementUI>(Errors::NotImplemented());
	}

	const int LocalUserNum = Services.Get<FAuthOSSAdapter>()->GetLocalUserNum(Params.LocalAccountId);
	if (LocalUserNum == INDEX_NONE)
	{
		return TOnlineResult<FDisplayAchievementUI>(Errors::InvalidUser());
	}

	if (Params.AchievementId.IsEmpty())
	{
		return TOnlineResult<FDisplayAchievementUI>(Errors::InvalidParams());
	}

	const bool bSuccess = ExternalUIInterface->ShowAchievementsUI(LocalUserNum);
	if (!bSuccess)
	{
		return TOnlineResult<FDisplayAchievementUI>(Errors::Unknown());
	}

	return TOnlineResult<FDisplayAchievementUI>(FDisplayAchievementUI::Result());
}

/* UE::Online */ }
