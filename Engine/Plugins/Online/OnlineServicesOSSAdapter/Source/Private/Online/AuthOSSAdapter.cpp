// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/AuthOSSAdapter.h"

#include "Misc/ScopeRWLock.h"
#include "Online/AuthErrors.h"
#include "Online/OnlineServicesOSSAdapter.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/DelegateAdapter.h"

#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"

#define UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_INITIAL_LOCAL_USER_NUM TEXT("InitialLocalUserNum")
#define UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO TEXT("AccountInfoOSSAdapter")
#define UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_EXTERNAL_AUTH_TOKEN_TRANSLATION_TRAITS TEXT("ExternalAuthTokenTranslationTraits")
#define UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_EXTERNAL_SERVER_AUTH_TICKET_TRANSLATION_TRAITS TEXT("ExternalServerAuthTicketTranslationTraits")

namespace UE::Online {

static TAutoConsoleVariable<bool> CVarAuthRegisterAccountLoggedInThroughOSSEnabled(
	TEXT("Online.AuthRegisterAccountLoggedInThroughOSSEnabled"),
	true,
	TEXT("When enabled, will register account logged in directly through OSSv1 instead of returning error"),
	ECVF_Default);

namespace {

ELoginStatus TranslateLoginStatus(::ELoginStatus::Type Status)
{
	switch (Status)
	{
	case ::ELoginStatus::NotLoggedIn:
		return ELoginStatus::NotLoggedIn;
	case ::ELoginStatus::UsingLocalProfile:
		return ELoginStatus::UsingLocalProfile;
	case ::ELoginStatus::LoggedIn:
		return ELoginStatus::LoggedIn;
	default:
		checkNoEntry();
		return ELoginStatus::NotLoggedIn;
	}
}

/* anonymous */ }

enum class EExternalAuthTokenTranslationFlags : uint8
{
	None = 0,
	TokenString = 1 << 0,
	TokenBinary = 1 << 1,
	AuthToken = 1 << 2,
};
ENUM_CLASS_FLAGS(EExternalAuthTokenTranslationFlags);

struct FExternalAuthTokenTranslationTraits
{
	FName ExternalLoginType;
	EExternalAuthTokenMethod Method;
	EExternalAuthTokenTranslationFlags Flags;
};

const FExternalAuthTokenTranslationTraits* GetExternalAuthTokenTranslationTraits(FName SubsystemType, EExternalAuthTokenMethod Method)
{
	static const TMap<FName, TArray<FExternalAuthTokenTranslationTraits>> SupportedExternalAuthTranslatorTraits = {
		{ TEXT("GDK"), { { ExternalLoginType::XblXstsToken, EExternalAuthTokenMethod::Primary, EExternalAuthTokenTranslationFlags::TokenString } } },
		{ PS4_SUBSYSTEM, { { ExternalLoginType::PsnIdToken, EExternalAuthTokenMethod::Primary, EExternalAuthTokenTranslationFlags::TokenString } } },
		{ TEXT("PS5"), { { ExternalLoginType::PsnIdToken, EExternalAuthTokenMethod::Primary, EExternalAuthTokenTranslationFlags::TokenString } } },
		{ NINTENDO_SUBSYSTEM, { { ExternalLoginType::NintendoNsaIdToken, EExternalAuthTokenMethod::Primary, EExternalAuthTokenTranslationFlags::AuthToken },
							  { ExternalLoginType::NintendoIdToken, EExternalAuthTokenMethod::Secondary, EExternalAuthTokenTranslationFlags::TokenString } } },
		{ STEAM_SUBSYSTEM, { { ExternalLoginType::SteamSessionTicket, EExternalAuthTokenMethod::Primary, EExternalAuthTokenTranslationFlags::TokenString } } },
	};

	if (const TArray<FExternalAuthTokenTranslationTraits>* TraitsArray = SupportedExternalAuthTranslatorTraits.Find(SubsystemType))
	{
		return TraitsArray->FindByPredicate([Method](const FExternalAuthTokenTranslationTraits& Value) { return Value.Method == Method; });
	}

	return nullptr;
}

struct FExternalServerAuthTicketTranslationTraits
{
	FName ExternalTicketType;
};

TDefaultErrorResultInternal<TSharedRef<FExternalServerAuthTicketTranslationTraits>> GetExternalServerAuthTicketTranslationTraits(FName SubsystemType)
{
	static const TMap<FName, TSharedRef<FExternalServerAuthTicketTranslationTraits>> SupportedExternalServerAuthTicketTranslationTraits = {
		{ TEXT("GDK"), MakeShared<FExternalServerAuthTicketTranslationTraits>(FExternalServerAuthTicketTranslationTraits{ ExternalServerAuthTicketType::XblXstsToken }) },
		{ PS4_SUBSYSTEM, MakeShared<FExternalServerAuthTicketTranslationTraits>(FExternalServerAuthTicketTranslationTraits{ ExternalServerAuthTicketType::PsnAuthCode }) },
		{ TEXT("PS5"), MakeShared<FExternalServerAuthTicketTranslationTraits>(FExternalServerAuthTicketTranslationTraits{ ExternalServerAuthTicketType::PsnAuthCode }) },
	};

	if (const TSharedRef<FExternalServerAuthTicketTranslationTraits>* Traits = SupportedExternalServerAuthTicketTranslationTraits.Find(SubsystemType))
	{
		return TDefaultErrorResultInternal<TSharedRef<FExternalServerAuthTicketTranslationTraits>>(*Traits);
	}

	return TDefaultErrorResultInternal<TSharedRef<FExternalServerAuthTicketTranslationTraits>>(Errors::NotImplemented());
}

TSharedPtr<FAccountInfoOSSAdapter> FAccountInfoRegistryOSSAdapter::Find(FPlatformUserId PlatformUserId) const
{
	return StaticCastSharedPtr<FAccountInfoOSSAdapter>(Super::Find(PlatformUserId));
} 

TSharedPtr<FAccountInfoOSSAdapter> FAccountInfoRegistryOSSAdapter::Find(FAccountId AccountId) const
{
	return StaticCastSharedPtr<FAccountInfoOSSAdapter>(Super::Find(AccountId));
}

void FAccountInfoRegistryOSSAdapter::Register(const TSharedRef<FAccountInfoOSSAdapter>& AccountInfoNULL)
{
	FWriteScopeLock Lock(IndexLock);
	DoRegister(AccountInfoNULL);
}

void FAccountInfoRegistryOSSAdapter::Unregister(FAccountId AccountId)
{
	if (TSharedPtr<FAccountInfoOSSAdapter> AccountInfoNULL = Find(AccountId))
	{
		FWriteScopeLock Lock(IndexLock);
		DoUnregister(AccountInfoNULL.ToSharedRef());
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "[FAccountInfoRegistryOSSAdapter::Unregister] Failed to find account [%ls].", *ToLogString(AccountId));
	}
}

void FAuthOSSAdapter::PostInitialize()
{
	Super::PostInitialize();

	if (IOnlineIdentityPtr Identity = GetIdentityInterface())
	{
		for (int LocalPlayerNum = 0; LocalPlayerNum < MAX_LOCAL_PLAYERS; ++LocalPlayerNum)
		{
			OnLoginStatusChangedHandle[LocalPlayerNum] = Identity->OnLoginStatusChangedDelegates[LocalPlayerNum].AddLambda(
				[WeakThis = TWeakPtr<FAuthOSSAdapter>(StaticCastSharedRef<FAuthOSSAdapter>(AsShared()))](int32 LocalUserNum, ::ELoginStatus::Type OldStatus, ::ELoginStatus::Type NewStatus, const FUniqueNetId& NetId)
				{
					if (TSharedPtr<FAuthOSSAdapter> PinnedThis = WeakThis.Pin())
					{
						PinnedThis->HandleLoginStatusChangedImplOp(FAuthHandleLoginStatusChangedImpl::Params{
							FPlatformMisc::GetPlatformUserForUserIndex(LocalUserNum),
							static_cast<FOnlineServicesOSSAdapter&>(PinnedThis->Services).GetAccountIdRegistry().FindOrAddHandle(NetId.AsShared()),
							TranslateLoginStatus(NewStatus) });
					}
				});
			if (FUniqueNetIdPtr NetId = Identity->GetUniquePlayerId(LocalPlayerNum))
			{
				HandleLoginStatusChangedImplOp(FAuthHandleLoginStatusChangedImpl::Params{
								FPlatformMisc::GetPlatformUserForUserIndex(LocalPlayerNum),
								static_cast<FOnlineServicesOSSAdapter&>(Services).GetAccountIdRegistry().FindOrAddHandle(NetId.ToSharedRef()),
								TranslateLoginStatus(Identity->GetLoginStatus(LocalPlayerNum)) });
			}			
		}
	}
}

void FAuthOSSAdapter::PreShutdown()
{
	if (IOnlineIdentityPtr Identity = GetIdentityInterface())
	{
		for (int LocalPlayerNum = 0; LocalPlayerNum < MAX_LOCAL_PLAYERS; ++LocalPlayerNum)
		{
			Identity->ClearOnLoginStatusChangedDelegate_Handle(LocalPlayerNum, OnLoginStatusChangedHandle[LocalPlayerNum]);
		}
	}

	Super::PreShutdown();
}

TOnlineAsyncOpHandle<FAuthLogin> FAuthOSSAdapter::Login(FAuthLogin::Params&& Params)
{
	TSharedRef<TOnlineAsyncOp<FAuthLogin>> Op = GetOp<FAuthLogin>(MoveTemp(Params));

	// Step 1: Setup operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const FAuthLogin::Params& Params = InAsyncOp.GetParams();

		const int32 InitialLocalUserNum = GetIdentityInterface()->GetLocalUserNumFromPlatformUserId(Params.PlatformUserId);
		if (InitialLocalUserNum == INDEX_NONE)
		{
			InAsyncOp.SetError(Errors::InvalidParams());
			return;
		}

		// Set initial user num on operation. Depending on the implementation the local user num
		// which completes login may be different from the one which started it.
		InAsyncOp.Data.Set<int32>(UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_INITIAL_LOCAL_USER_NUM, InitialLocalUserNum);

		if (AccountInfoRegistryOSSAdapter.Find(Params.PlatformUserId))
		{
			InAsyncOp.SetError(Errors::Auth::AlreadyLoggedIn());
			return;
		}

		// Todo: Handle platforms that allow calling login again with a different login type.

		TSharedPtr<FAccountInfoOSSAdapter> AccountInfoOSSAdapter = MakeShared<FAccountInfoOSSAdapter>();
		AccountInfoOSSAdapter->LoginStatus = ELoginStatus::NotLoggedIn;

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoOSSAdapter>>(UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO, AccountInfoOSSAdapter.ToSharedRef());
	})
	// Step 2: Login to OSSv1 identity interface.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoOSSAdapter>& AccountInfoOSSAdapter = GetOpDataChecked<TSharedRef<FAccountInfoOSSAdapter>>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO);
		const int InitialLocalUserNum = GetOpDataChecked<int32>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_INITIAL_LOCAL_USER_NUM);

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();
		MakeMulticastAdapter(this, GetIdentityInterface()->OnLoginCompleteDelegates[InitialLocalUserNum],
		[this, AccountInfoOSSAdapter, WeakOp = InAsyncOp.AsWeak(), Promise = MoveTemp(Promise)](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error) mutable
		{
			if (TSharedPtr<TOnlineAsyncOp<FAuthLogin>> Op = WeakOp.Pin())
			{
				if (bWasSuccessful)
				{
					// Sanity check UserId
					if (!UserId.IsValid())
					{
						UE_LOGF(LogOnlineServices, Warning, "[FAuthOSSAdapter::Login][%ls] Failure: Userid was invalid.", *GetSubsystem().GetSubsystemName().ToString());
						Op->SetError(Errors::Unknown());
					}
					else
					{
						AccountInfoOSSAdapter->AccountId = static_cast<FOnlineServicesOSSAdapter&>(Services).GetAccountIdRegistry().FindOrAddHandle(UserId.AsShared());
						AccountInfoOSSAdapter->UniqueNetId = UserId.AsShared();
						AccountInfoOSSAdapter->LocalUserNum = LocalUserNum;
						AccountInfoOSSAdapter->PlatformUserId = GetIdentityInterface()->GetPlatformUserIdFromLocalUserNum(LocalUserNum);
					}
				}
				else
				{
					UE_LOGF(LogOnlineServices, Warning, "[FAuthOSSAdapter::Login][%ls] Failure: Failed to login to OSSv1 identity interface. Error %ls", *GetSubsystem().GetSubsystemName().ToString(), *Error);
					FOnlineError V2Error = Errors::Unknown(); // TODO: V1 to V2 error conversion/error from string conversion
					Op->SetError(MoveTemp(V2Error));
				}
			}

			Promise.EmplaceValue();
		});

		if (InAsyncOp.GetParams().CredentialsType == LoginCredentialsType::Auto)
		{
			GetIdentityInterface()->AutoLogin(InitialLocalUserNum);
		}
		else
		{
			FOnlineAccountCredentials Credentials;
			Credentials.Type = InAsyncOp.GetParams().CredentialsType.ToString();
			Credentials.Id = InAsyncOp.GetParams().CredentialsId;
			Credentials.Token = InAsyncOp.GetParams().CredentialsToken.Get<FString>(); // TODO: handle binary token

			GetIdentityInterface()->Login(InitialLocalUserNum, Credentials);
		}
		return Future;
	})
	// Step 4: Fetch dependent data.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoOSSAdapter>& AccountInfoOSSAdapter = GetOpDataChecked<TSharedRef<FAccountInfoOSSAdapter>>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO);

		if (!PopulateAttributes(*AccountInfoOSSAdapter))
		{
			InAsyncOp.SetError(Errors::Unknown());

			// Logout.
			TPromise<void> Promise;
			TFuture<void> Future = Promise.GetFuture();
			MakeMulticastAdapter(this, GetIdentityInterface()->OnLogoutCompleteDelegates[AccountInfoOSSAdapter->LocalUserNum],
			[this, AccountInfoOSSAdapter, Promise = MoveTemp(Promise)](int32 LocalUserNum, bool bWasSuccessful) mutable
			{
				if (!bWasSuccessful)
				{
					UE_LOGF(LogOnlineServices, Warning, "[FAuthOSSAdapter::Login][%ls] Failed to logout account [%ls].", *GetSubsystem().GetSubsystemName().ToString(), *ToLogString(AccountInfoOSSAdapter->AccountId));
				}
				Promise.EmplaceValue();
			});

			GetIdentityInterface()->Logout(AccountInfoOSSAdapter->LocalUserNum);
			return Future;
		}

		return MakeFulfilledPromise<void>().GetFuture();
	})
	// Step 6: bookkeeping and notifications.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoOSSAdapter>& AccountInfoOSSAdapter = GetOpDataChecked<TSharedRef<FAccountInfoOSSAdapter>>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO);

		AccountInfoOSSAdapter->LoginStatus = ELoginStatus::LoggedIn;
		AccountInfoRegistryOSSAdapter.Register(AccountInfoOSSAdapter);

		UE_LOGF(LogOnlineServices, Log, "[FAuthOSSAdapter::Login][%ls] Successfully logged in as [%ls]", *GetSubsystem().GetSubsystemName().ToString(), *ToLogString(AccountInfoOSSAdapter->AccountId));
		OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoOSSAdapter, AccountInfoOSSAdapter->LoginStatus });

		InAsyncOp.SetResult(FAuthLogin::Result{ AccountInfoOSSAdapter });
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthLogout> FAuthOSSAdapter::Logout(FAuthLogout::Params&& Params)
{
	TSharedRef<TOnlineAsyncOp<FAuthLogout>> Op = GetOp<FAuthLogout>(MoveTemp(Params));

	// Step 1: Setup operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthLogout>& InAsyncOp)
	{
		const FAuthLogout::Params& Params = InAsyncOp.GetParams();

		TSharedPtr<FAccountInfoOSSAdapter> AccountInfoOSSAdapter = AccountInfoRegistryOSSAdapter.Find(Params.LocalAccountId);
		if (!AccountInfoOSSAdapter)
		{
			InAsyncOp.SetError(Errors::InvalidUser());
			return;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoOSSAdapter>>(UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO, AccountInfoOSSAdapter.ToSharedRef());
	})
	// Step 2: Logout the user.
	.Then([this](TOnlineAsyncOp<FAuthLogout>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoOSSAdapter>& AccountInfoOSSAdapter = GetOpDataChecked<TSharedRef<FAccountInfoOSSAdapter>>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO);

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();
		MakeMulticastAdapter(this, GetIdentityInterface()->OnLogoutCompleteDelegates[AccountInfoOSSAdapter->LocalUserNum],
		[this, AccountInfoOSSAdapter, Promise = MoveTemp(Promise)](int32 LocalUserNum, bool bWasSuccessful) mutable
		{
			if (!bWasSuccessful)
			{
				UE_LOGF(LogOnlineServices, Warning, "[FAuthOSSAdapter::Logout][%ls] Failed to logout account [%ls].", *GetSubsystem().GetSubsystemName().ToString(), *ToLogString(AccountInfoOSSAdapter->AccountId));
			}
			Promise.EmplaceValue();
		});

		GetIdentityInterface()->Logout(AccountInfoOSSAdapter->LocalUserNum);
		return Future;
	})
	// Step 3: bookkeeping and notifications.
	.Then([this](TOnlineAsyncOp<FAuthLogout>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoOSSAdapter>& AccountInfoOSSAdapter = GetOpDataChecked<TSharedRef<FAccountInfoOSSAdapter>>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO);

		UE_LOGF(LogOnlineServices, Log, "[FAuthOSSAdapter::Logout][%ls] Successfully logged out [%ls]", *GetSubsystem().GetSubsystemName().ToString(), *ToLogString(AccountInfoOSSAdapter->AccountId));
		AccountInfoOSSAdapter->LoginStatus = ELoginStatus::NotLoggedIn;
		OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoOSSAdapter, AccountInfoOSSAdapter->LoginStatus });
		AccountInfoRegistryOSSAdapter.Unregister(AccountInfoOSSAdapter->AccountId);
		InAsyncOp.SetResult(FAuthLogout::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthQueryExternalServerAuthTicket> FAuthOSSAdapter::QueryExternalServerAuthTicket(FAuthQueryExternalServerAuthTicket::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthQueryExternalServerAuthTicket> Op = GetOp<FAuthQueryExternalServerAuthTicket>(MoveTemp(Params));

	// Step 1: Setup operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthQueryExternalServerAuthTicket>& InAsyncOp)
	{
		const FAuthQueryExternalServerAuthTicket::Params& Params = InAsyncOp.GetParams();

		// Check for supported translator
		TDefaultErrorResultInternal<TSharedRef<FExternalServerAuthTicketTranslationTraits>> TranslationTraits = GetExternalServerAuthTicketTranslationTraits(GetSubsystem().GetSubsystemName());
		if (!TranslationTraits.IsOk())
		{
			InAsyncOp.SetError(MoveTemp(TranslationTraits.GetErrorValue()));
			return;
		}

		// Set translator traits on operation.
		InAsyncOp.Data.Set<TSharedRef<FExternalServerAuthTicketTranslationTraits>>(UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_EXTERNAL_SERVER_AUTH_TICKET_TRANSLATION_TRAITS, TranslationTraits.GetOkValue());

		// Look up logged in user.
		TSharedPtr<FAccountInfoOSSAdapter> AccountInfoOSSAdapter = AccountInfoRegistryOSSAdapter.Find(Params.LocalAccountId);
		if (!AccountInfoOSSAdapter)
		{
			InAsyncOp.SetError(Errors::InvalidUser());
			return;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoOSSAdapter>>(UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO, AccountInfoOSSAdapter.ToSharedRef());
	})
	// Step 2: Fetch OSSv1 auth ticket data and signal result.
	.Then([this](TOnlineAsyncOp<FAuthQueryExternalServerAuthTicket>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoOSSAdapter>& AccountInfoOSSAdapter = GetOpDataChecked<TSharedRef<FAccountInfoOSSAdapter>>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO);
		const TSharedRef<FExternalServerAuthTicketTranslationTraits>& ExternalServerAuthTicketTranslationTraits = GetOpDataChecked<TSharedRef<FExternalServerAuthTicketTranslationTraits>>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_EXTERNAL_AUTH_TOKEN_TRANSLATION_TRAITS);

		InAsyncOp.SetResult(FAuthQueryExternalServerAuthTicket::Result{FExternalServerAuthTicket{ExternalServerAuthTicketTranslationTraits->ExternalTicketType, GetIdentityInterface()->GetAuthToken(AccountInfoOSSAdapter->LocalUserNum)}});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthQueryExternalAuthToken> FAuthOSSAdapter::QueryExternalAuthToken(FAuthQueryExternalAuthToken::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthQueryExternalAuthToken> Op = GetOp<FAuthQueryExternalAuthToken>(MoveTemp(Params));

	// Step 1: Setup operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthQueryExternalAuthToken>& InAsyncOp)
	{
		const FAuthQueryExternalAuthToken::Params& Params = InAsyncOp.GetParams();

		// Check for supported translator
		const FExternalAuthTokenTranslationTraits* TranslationTraits = GetExternalAuthTokenTranslationTraits(GetSubsystem().GetSubsystemName(), Params.Method);
		if (TranslationTraits == nullptr)
		{
			InAsyncOp.SetError(Errors::NotImplemented());
			return;
		}

		// Set translator traits on operation.
		InAsyncOp.Data.Set<const FExternalAuthTokenTranslationTraits*>(UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_EXTERNAL_AUTH_TOKEN_TRANSLATION_TRAITS, TranslationTraits);

		// Look up logged in user.
		TSharedPtr<FAccountInfoOSSAdapter> AccountInfoOSSAdapter = AccountInfoRegistryOSSAdapter.Find(Params.LocalAccountId);
		if (!AccountInfoOSSAdapter)
		{
			InAsyncOp.SetError(Errors::InvalidUser());
			return;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoOSSAdapter>>(UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO, AccountInfoOSSAdapter.ToSharedRef());
	})
	// Step 2: Fetch OSSv1 token data.
	.Then([this](TOnlineAsyncOp<FAuthQueryExternalAuthToken>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoOSSAdapter>& AccountInfoOSSAdapter = GetOpDataChecked<TSharedRef<FAccountInfoOSSAdapter>>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO);
		const FExternalAuthTokenTranslationTraits* ExternalAuthTokenTranslationTraits = GetOpDataChecked<const FExternalAuthTokenTranslationTraits*>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_EXTERNAL_AUTH_TOKEN_TRANSLATION_TRAITS);

		TPromise<FString> Promise;
		TFuture<FString> Future = Promise.GetFuture();

		if (EnumHasAnyFlags(ExternalAuthTokenTranslationTraits->Flags, EExternalAuthTokenTranslationFlags::AuthToken))
		{
			Promise.EmplaceValue(GetIdentityInterface()->GetAuthToken(AccountInfoOSSAdapter->LocalUserNum));
		}
		else if (EnumHasAnyFlags(ExternalAuthTokenTranslationTraits->Flags, EExternalAuthTokenTranslationFlags::TokenString | EExternalAuthTokenTranslationFlags::TokenBinary))
		{
			GetIdentityInterface()->GetLinkedAccountAuthToken(AccountInfoOSSAdapter->LocalUserNum, FString(), *MakeDelegateAdapter(this,
			[this, WeakOp = InAsyncOp.AsWeak(), Promise = MoveTemp(Promise), ExternalAuthTokenTranslationTraits](int32 LocalUserNum, bool bWasSuccessful, const ::FExternalAuthToken& AuthToken) mutable
			{
				if (TSharedPtr<TOnlineAsyncOp<FAuthQueryExternalAuthToken>> Op = WeakOp.Pin())
				{
					if (bWasSuccessful)
					{
						if (EnumHasAnyFlags(ExternalAuthTokenTranslationTraits->Flags, EExternalAuthTokenTranslationFlags::TokenString))
						{
							if (AuthToken.HasTokenString())
							{
								Promise.EmplaceValue(AuthToken.TokenString);
							}
							else
							{
								UE_LOGF(LogOnlineServices, Warning, "[FAuthOSSAdapter::QueryExternalAuthToken][%ls] Failed: Token string not found.", *GetSubsystem().GetSubsystemName().ToString());
								Op->SetError(Errors::Unknown());
								Promise.EmplaceValue();
							}
						}
						else if (EnumHasAnyFlags(ExternalAuthTokenTranslationTraits->Flags, EExternalAuthTokenTranslationFlags::TokenBinary))
						{
							if (AuthToken.HasTokenData())
							{
								// Convert binary to hex representation.
								Promise.EmplaceValue(FString::FromHexBlob(AuthToken.TokenData.GetData(), AuthToken.TokenData.Num()));
							}
							else
							{
								UE_LOGF(LogOnlineServices, Warning, "[FAuthOSSAdapter::QueryExternalAuthToken][%ls] Failed: Token binary data not found.", *GetSubsystem().GetSubsystemName().ToString());
								Op->SetError(Errors::Unknown());
								Promise.EmplaceValue();
							}
						}
						else
						{
							checkNoEntry();
							Op->SetError(Errors::Unknown());
							Promise.EmplaceValue();
						}
					}
					else
					{
						UE_LOGF(LogOnlineServices, Warning, "[FAuthOSSAdapter::QueryExternalAuthToken][%ls] GetLinkedAccountAuthToken failed.", *GetSubsystem().GetSubsystemName().ToString());
						Op->SetError(Errors::Unknown());
						Promise.EmplaceValue();
					}
				}
				else
				{
					Promise.EmplaceValue();
				}
			}));
		}
		else
		{
			// Getting here means the traits were setup incorrectly.
			InAsyncOp.SetError(Errors::Unknown());
			Promise.EmplaceValue();
		}

		return Future;
	})
	// Step 3: Signal valid result.
	.Then([this](TOnlineAsyncOp<FAuthQueryExternalAuthToken>& InAsyncOp, FString&& ResolvedExternalAuthToken)
	{
		const FExternalAuthTokenTranslationTraits* ExternalAuthTokenTranslationTraits = GetOpDataChecked<const FExternalAuthTokenTranslationTraits*>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_EXTERNAL_AUTH_TOKEN_TRANSLATION_TRAITS);
		InAsyncOp.SetResult(FAuthQueryExternalAuthToken::Result{FExternalAuthToken{ExternalAuthTokenTranslationTraits->ExternalLoginType, MoveTemp(ResolvedExternalAuthToken)}});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

FUniqueNetIdPtr FAuthOSSAdapter::GetUniqueNetId(FAccountId AccountId) const
{
	check(!AccountId.IsValid() || AccountId.GetOnlineServicesType() == GetOnlineServicesOSSAdapter().GetServicesProvider());

	return GetOnlineServicesOSSAdapter().GetAccountIdRegistry().GetIdValue(AccountId);
}

FAccountId FAuthOSSAdapter::GetAccountId(const FUniqueNetIdRef& UniqueNetId) const
{
	check(UniqueNetId->GetType() == GetSubsystem().GetSubsystemName());
	
	return GetOnlineServicesOSSAdapter().GetAccountIdRegistry().FindOrAddHandle(UniqueNetId);
}

int32 FAuthOSSAdapter::GetLocalUserNum(FAccountId AccountId) const
{
	TSharedPtr<FAccountInfoOSSAdapter> AccountInfoOSSAdapter = AccountInfoRegistryOSSAdapter.Find(AccountId);
	return AccountInfoOSSAdapter ? AccountInfoOSSAdapter->LocalUserNum : INDEX_NONE;
}

#if !UE_BUILD_SHIPPING
void FAuthOSSAdapter::CheckMetadata()
{
	// Metadata sanity check.
	ToLogString(FAuthHandleLoginStatusChangedImpl::Params());
	ToLogString(FAuthHandleLoginStatusChangedImpl::Result());
	Meta::VisitFields(FAuthHandleLoginStatusChangedImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthHandleLoginStatusChangedImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
}
#endif

const FAccountInfoRegistry& FAuthOSSAdapter::GetAccountInfoRegistry() const
{
	return AccountInfoRegistryOSSAdapter;
}

const FOnlineServicesOSSAdapter& FAuthOSSAdapter::GetOnlineServicesOSSAdapter() const
{
	return const_cast<FAuthOSSAdapter*>(this)->GetOnlineServicesOSSAdapter();
}

FOnlineServicesOSSAdapter& FAuthOSSAdapter::GetOnlineServicesOSSAdapter()
{
	return static_cast<FOnlineServicesOSSAdapter&>(Services);
}

const IOnlineSubsystem& FAuthOSSAdapter::GetSubsystem() const
{
	return GetOnlineServicesOSSAdapter().GetSubsystem();
}

IOnlineIdentityPtr FAuthOSSAdapter::GetIdentityInterface() const
{
	return GetSubsystem().GetIdentityInterface();
}

TOnlineAsyncOpHandle<FAuthHandleLoginStatusChangedImpl> FAuthOSSAdapter::HandleLoginStatusChangedImplOp(FAuthHandleLoginStatusChangedImpl::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthHandleLoginStatusChangedImpl> Op = GetOp<FAuthHandleLoginStatusChangedImpl>(MoveTemp(Params));

	// Step 1: Set up operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthHandleLoginStatusChangedImpl>& InAsyncOp)
	{
		const FAuthHandleLoginStatusChangedImpl::Params& Params = InAsyncOp.GetParams();
		
		TSharedPtr<FAccountInfoOSSAdapter> AccountInfoOSSAdapter = AccountInfoRegistryOSSAdapter.Find(Params.AccountId);

		if (!AccountInfoOSSAdapter)
		{
			if (CVarAuthRegisterAccountLoggedInThroughOSSEnabled.GetValueOnAnyThread())
			{
				// In case of Login get called without going through the OSSv2 OSS Adapter, add logged in user to AccountInfoRegistry as well
				AccountInfoOSSAdapter = MakeShared<FAccountInfoOSSAdapter>();
				AccountInfoOSSAdapter->AccountId = Params.AccountId;
				AccountInfoOSSAdapter->PlatformUserId = Params.PlatformUserId;
				AccountInfoOSSAdapter->LoginStatus = ELoginStatus::NotLoggedIn;
				AccountInfoOSSAdapter->LocalUserNum = FPlatformMisc::GetUserIndexForPlatformUser(Params.PlatformUserId);
				AccountInfoOSSAdapter->UniqueNetId = GetIdentityInterface()->GetUniquePlayerId(AccountInfoOSSAdapter->LocalUserNum);
				if (!PopulateAttributes(*AccountInfoOSSAdapter))
				{
					InAsyncOp.SetError(Errors::InvalidUser());
					return;
				}
				AccountInfoRegistryOSSAdapter.Register(AccountInfoOSSAdapter.ToSharedRef());
			}
			else
			{
				InAsyncOp.SetError(Errors::InvalidUser());
				return;
			}
		}

		// Don't send duplicate notification during login
		if (AccountInfoOSSAdapter->LoginStatus == Params.NewLoginStatus)
		{
			UE_LOGF(LogOnlineServices, Log, "[FAuthOSSAdapter::HandleLoginStatusChangedImplOp][%ls] Login status has not changed. Ignoring event. [%ls] Current Status: %ls",
				*GetSubsystem().GetSubsystemName().ToString(), *ToLogString(AccountInfoOSSAdapter->AccountId), *ToLogString(Params.NewLoginStatus));
			InAsyncOp.SetResult(FAuthHandleLoginStatusChangedImpl::Result{});
			return;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoOSSAdapter>>(UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO, AccountInfoOSSAdapter.ToSharedRef());
	})
	// Step 2: Update status and notify.
	.Then([this](TOnlineAsyncOp<FAuthHandleLoginStatusChangedImpl>& InAsyncOp)
	{
		const FAuthHandleLoginStatusChangedImpl::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FAccountInfoOSSAdapter>& AccountInfoOSSAdapter = GetOpDataChecked<TSharedRef<FAccountInfoOSSAdapter>>(InAsyncOp, UE_ONLINE_AUTH_OSS_ADAPTER_KEY_NAME_ACCOUNT_INFO);

		UE_LOGF(LogOnlineServices, Log, "[FAuthOSSAdapter::HandleLoginStatusChangedImplOp][%ls] Login status changed for account [%ls]: %ls => %ls.",
			*GetSubsystem().GetSubsystemName().ToString(), *ToLogString(AccountInfoOSSAdapter->AccountId), *ToLogString(AccountInfoOSSAdapter->LoginStatus), *ToLogString(Params.NewLoginStatus));
		AccountInfoOSSAdapter->LoginStatus = Params.NewLoginStatus;
		OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoOSSAdapter, AccountInfoOSSAdapter->LoginStatus });
		InAsyncOp.SetResult(FAuthHandleLoginStatusChangedImpl::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

bool FAuthOSSAdapter::PopulateAttributes(FAccountInfoOSSAdapter& AccountInfoOSSAdapter) const
{
	if (AccountInfoOSSAdapter.UniqueNetId && AccountInfoOSSAdapter.UniqueNetId->IsValid())
	{
		TSharedPtr<FUserOnlineAccount> UserOnlineAccount = GetIdentityInterface()->GetUserAccount(*AccountInfoOSSAdapter.UniqueNetId);
		if (UserOnlineAccount)
		{
			AccountInfoOSSAdapter.Attributes.Emplace(AccountAttributeData::DisplayName, UserOnlineAccount->GetDisplayName());
			return true;
		}

		FString Displayname = GetIdentityInterface()->GetPlayerNickname(*AccountInfoOSSAdapter.UniqueNetId);

		if (!Displayname.IsEmpty())
		{
			AccountInfoOSSAdapter.Attributes.Emplace(AccountAttributeData::DisplayName, GetIdentityInterface()->GetPlayerNickname(*AccountInfoOSSAdapter.UniqueNetId));
			return true;
		}
	}

	UE_LOGF(LogOnlineServices, Warning, "[FAuthOSSAdapter::Login][%ls] Failure: Failed to find UserOnlineAccount for account %ls", *GetSubsystem().GetSubsystemName().ToString(), *ToLogString(AccountInfoOSSAdapter.AccountId));

	return false;
}

/* UE::Online */ }
