// Copyright Epic Games, Inc. All Rights Reserved.
#include "Online/CommerceOSSAdapter.h"
#include "Online/DelegateAdapter.h"
#include "Online/ErrorsOSSAdapter.h"
#include "Online/OnlineServicesOSSAdapter.h"

namespace UE::Online {
FOffer OfferV1toV2(FOnlineStoreOfferRef OfferV1)
{
	FOffer RetOffer;

	RetOffer.OfferId = OfferV1->OfferId;
	RetOffer.Title = OfferV1->Title;
	RetOffer.Description = OfferV1->Description;
	RetOffer.LongDescription = OfferV1->LongDescription;
	RetOffer.CurrencyCode = OfferV1->CurrencyCode;
	RetOffer.FormattedRegularPrice = OfferV1->RegularPriceText;
	RetOffer.RegularPrice = OfferV1->RegularPrice;
	RetOffer.FormattedPrice = OfferV1->PriceText;
	RetOffer.Price = OfferV1->NumericPrice;
	RetOffer.ReleaseDate = OfferV1->ReleaseDate;
	RetOffer.ExpirationDate = OfferV1->ExpirationDate;
	RetOffer.AdditionalData = OfferV1->DynamicFields;

	// these two fields don't have v1 equivalents- use default values (-1 for purchase limit implying no purchase limit, 2 for price decimal point since most currency types use this)
	RetOffer.PurchaseLimit = -1;
	RetOffer.PriceDecimalPoint = 2;

	return RetOffer;
}

TArray<FEntitlement> EntitlementV1toV2(const FPurchaseReceipt& Receipt)
{
	TArray<FEntitlement> Entitlements;
	
	if (Receipt.TransactionState == EPurchaseTransactionState::Purchased || Receipt.TransactionState == EPurchaseTransactionState::Restored)
	{
		for (const FPurchaseReceipt::FReceiptOfferEntry& ReceiptEntry : Receipt.ReceiptOffers)
		{
			for (const FPurchaseReceipt::FLineItemInfo& ReceiptEntryLine : ReceiptEntry.LineItems)
			{
				FEntitlement& Entitlement = Entitlements.Emplace_GetRef();

				Entitlement.EntitlementId = ReceiptEntryLine.UniqueId;
				Entitlement.ProductId = ReceiptEntry.OfferId;
				Entitlement.bRedeemed = !ReceiptEntryLine.IsRedeemable();
				Entitlement.Quantity = ReceiptEntry.Quantity;
				// does EntitlementType match any fields we have?
			}
		}
	}

	return Entitlements;
}

void FCommerceOSSAdapter::PostInitialize()
{	
	Super::PostInitialize();

	Auth = Services.Get<FAuthOSSAdapter>();
	IOnlineStoreV2Ptr StorePtr = GetStoreInterface();
	IOnlinePurchasePtr PurchasePtr = GetPurchaseInterface();
	check(StorePtr.IsValid());
	check(PurchasePtr.IsValid());

	MakeMulticastAdapter(this, GetPurchaseInterface()->OnUnexpectedPurchaseReceiptDelegates, [this](const FUniqueNetId& User)
	{
		FCommerceOnPurchaseComplete Event;
		Event.LocalAccountId = Auth->GetAccountId(User.AsShared());
		OnPurchaseCompletedEvent.Broadcast(MoveTemp(Event));

		return false; // don't unbind
	});
}

void FCommerceOSSAdapter::PreShutdown()
{
	Super::PreShutdown();
	// Don't need to unbind OnUnexpectedPurchaseReceipt- will be done when the weak ptr expires
}

TOnlineAsyncOpHandle<FCommerceQueryOffers> FCommerceOSSAdapter::QueryOffers(FCommerceQueryOffers::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryOffers> Op = GetOp<FCommerceQueryOffers>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FCommerceQueryOffers>& InAsyncOp)
	{
		IOnlineStoreV2Ptr StorePtr = GetStoreInterface();
		FUniqueNetIdPtr NetId = Auth->GetUniqueNetId(InAsyncOp.GetParams().LocalAccountId);
		StorePtr->QueryOffersByFilter(*NetId, FOnlineStoreFilter(), *MakeDelegateAdapter(InAsyncOp, [this, WeakOp = InAsyncOp.AsWeak()](bool bWasSuccessful, const TArray<FUniqueOfferId>& OfferIds, const FString& Error)
		{
			if (bWasSuccessful)
			{
				WeakOp.Pin()->SetResult({});
			}
			else
			{
				// todo: string to error
				WeakOp.Pin()->SetError(Errors::Unknown());
			}
		}));
	})
	.Enqueue(GetSerialQueue());
	

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryOffersById> FCommerceOSSAdapter::QueryOffersById(FCommerceQueryOffersById::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryOffersById> Op = GetOp<FCommerceQueryOffersById>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FCommerceQueryOffersById>& InAsyncOp)
	{
		IOnlineStoreV2Ptr StorePtr = GetStoreInterface();
		FUniqueNetIdPtr NetId = Auth->GetUniqueNetId(InAsyncOp.GetParams().LocalAccountId);
		StorePtr->QueryOffersById(*NetId, InAsyncOp.GetParams().OfferIds, *MakeDelegateAdapter(InAsyncOp, [this, WeakOp = InAsyncOp.AsWeak()](bool bWasSuccessful, const TArray<FUniqueOfferId>& OfferIds, const FString& Error)
		{
			if (bWasSuccessful)
			{
				WeakOp.Pin()->SetResult({ OfferIds });
			}
			else
			{
				// todo: string to error
				WeakOp.Pin()->SetError(Errors::Unknown());
			}
		}));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineResult<FCommerceGetOffers> FCommerceOSSAdapter::GetOffers(FCommerceGetOffers::Params&& Params)
{
	IOnlineStoreV2Ptr StorePtr = GetStoreInterface();
	TArray<FOnlineStoreOfferRef> OutOssOffers;
	TArray<FOffer> OutOffers;
	StorePtr->GetOffers(OutOssOffers);

	if (OutOssOffers.IsEmpty())
	{
		return TOnlineResult<FCommerceGetOffers>(Errors::NotFound());
	}

	for (const FOnlineStoreOfferRef& OssOffer : OutOssOffers)
	{
		OutOffers.Add(OfferV1toV2(OssOffer));
	}

	return TOnlineResult<FCommerceGetOffers>({OutOffers});
}

TOnlineResult<FCommerceGetOffersById> FCommerceOSSAdapter::GetOffersById(FCommerceGetOffersById::Params&& Params)
{
	IOnlineStoreV2Ptr StorePtr = GetStoreInterface();
	TArray<FOffer> OutOffers;

	for (const FOfferId& OfferId : Params.OfferIds)
	{
		TSharedPtr<FOnlineStoreOffer> Offer = StorePtr->GetOffer(OfferId);
		if (Offer.IsValid())
		{
			OutOffers.Add(OfferV1toV2(Offer.ToSharedRef()));
		}
	}

	if (OutOffers.IsEmpty())
	{
		return TOnlineResult<FCommerceGetOffersById>(Errors::NotFound());
	}

	return TOnlineResult<FCommerceGetOffersById>({ OutOffers });
}

TOnlineAsyncOpHandle<FCommerceShowStoreUI> FCommerceOSSAdapter::ShowStoreUI(FCommerceShowStoreUI::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceShowStoreUI> Op = GetOp<FCommerceShowStoreUI>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FCommerceShowStoreUI>& InAsyncOp)
	{
		if(!GetExternalUIInterface().IsValid())
		{
			InAsyncOp.SetError(Errors::NotImplemented());
			return;
		}

		GetExternalUIInterface()->ShowStoreUI(Auth->GetLocalUserNum(InAsyncOp.GetParams().LocalAccountId), FShowStoreParams());
		InAsyncOp.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceCheckout> FCommerceOSSAdapter::Checkout(FCommerceCheckout::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceCheckout> Op = GetOp<FCommerceCheckout>(MoveTemp(Params)); 
	
	Op->Then([this](TOnlineAsyncOp<FCommerceCheckout>& InAsyncOp)
	{
		FPurchaseCheckoutRequest CheckoutRequest;

		for (const FPurchaseOffer& Offer : InAsyncOp.GetParams().Offers)
		{
			CheckoutRequest.AddPurchaseOffer(FOfferNamespace(), Offer.OfferId, Offer.Quantity);
		}

		GetPurchaseInterface()->Checkout(*Auth->GetUniqueNetId(InAsyncOp.GetParams().LocalAccountId), CheckoutRequest, *MakeDelegateAdapter(InAsyncOp, [this, WeakOp = InAsyncOp.AsWeak()](const ::FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt) mutable
		{
			if (Result.WasSuccessful())
			{
				FCommerceCheckout::Result OpResult;
				OpResult.TransactionId = Receipt->TransactionId;
				WeakOp.Pin()->SetResult(MoveTemp(OpResult));

				FCommerceOnPurchaseComplete Event;
				Event.LocalAccountId = WeakOp.Pin()->GetParams().LocalAccountId;
				OnPurchaseCompletedEvent.Broadcast(MoveTemp(Event));
			}
			else
			{
				WeakOp.Pin()->SetError(Errors::FromOssError(Result));
			}
		}));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryTransactionEntitlements> FCommerceOSSAdapter::QueryTransactionEntitlements(FCommerceQueryTransactionEntitlements::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryTransactionEntitlements> Op = GetOp<FCommerceQueryTransactionEntitlements>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FCommerceQueryTransactionEntitlements>& InAsyncOp)
	{
		GetPurchaseInterface()->QueryReceipts(*Auth->GetUniqueNetId(InAsyncOp.GetParams().LocalAccountId), true, *MakeDelegateAdapter(InAsyncOp, [this, WeakOp = InAsyncOp.AsWeak()](const ::FOnlineError& Result) mutable
		{
			if (Result.WasSuccessful())
			{
				FCommerceQueryTransactionEntitlements::Result OpResult;
				TArray<FPurchaseReceipt> OutReceipts;
				GetPurchaseInterface()->GetReceipts(*Auth->GetUniqueNetId(WeakOp.Pin()->GetParams().LocalAccountId), OutReceipts);

				for (const FPurchaseReceipt& Receipt : OutReceipts)
				{
					if (Receipt.TransactionId == WeakOp.Pin()->GetParams().TransactionId)
					{
						TArray<FEntitlement> Entitlements = EntitlementV1toV2(Receipt);
						for (const FEntitlement& Entitlement : Entitlements)
						{
							OpResult.Entitlements.Add(Entitlement);
						}
					}
				}

				WeakOp.Pin()->SetResult(MoveTemp(OpResult));
			}
			else
			{
				WeakOp.Pin()->SetError(Errors::FromOssError(Result));
			}
		}));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryEntitlements> FCommerceOSSAdapter::QueryEntitlements(FCommerceQueryEntitlements::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryEntitlements> Op = GetOp<FCommerceQueryEntitlements>(MoveTemp(Params));
	
	Op->Then([this](TOnlineAsyncOp<FCommerceQueryEntitlements>& InAsyncOp)
	{
		GetPurchaseInterface()->QueryReceipts(*Auth->GetUniqueNetId(InAsyncOp.GetParams().LocalAccountId), true, *MakeDelegateAdapter(InAsyncOp, [this, WeakOp = InAsyncOp.AsWeak()](const ::FOnlineError& Result) mutable
		{
			if (Result.WasSuccessful())
			{
				WeakOp.Pin()->SetResult({});
			}
			else
			{
				WeakOp.Pin()->SetError(Errors::FromOssError(Result));
			}
		}));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineResult<FCommerceGetEntitlements> FCommerceOSSAdapter::GetEntitlements(FCommerceGetEntitlements::Params&& Params)
{
	FCommerceGetEntitlements::Result Result;
	TArray<FPurchaseReceipt> OutReceipts;
	GetPurchaseInterface()->GetReceipts(*Auth->GetUniqueNetId(Params.LocalAccountId), OutReceipts);

	for (const FPurchaseReceipt& Receipt : OutReceipts)
	{
		TArray<FEntitlement> Entitlements = EntitlementV1toV2(Receipt);
		for (const FEntitlement& Entitlement : Entitlements)
		{
			Result.Entitlements.Add(Entitlement);
		}
	}

	return TOnlineResult<FCommerceGetEntitlements>(MoveTemp(Result));
}

TOnlineAsyncOpHandle<FCommerceRedeemEntitlement> FCommerceOSSAdapter::RedeemEntitlement(FCommerceRedeemEntitlement::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceRedeemEntitlement> Op = GetOp<FCommerceRedeemEntitlement>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceRetrieveS2SToken> FCommerceOSSAdapter::RetrieveS2SToken(FCommerceRetrieveS2SToken::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceRetrieveS2SToken> Op = GetOp<FCommerceRetrieveS2SToken>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

const FOnlineServicesOSSAdapter& FCommerceOSSAdapter::GetOnlineServicesOSSAdapter() const
{
	return const_cast<FCommerceOSSAdapter*>(this)->GetOnlineServicesOSSAdapter();
}

FOnlineServicesOSSAdapter& FCommerceOSSAdapter::GetOnlineServicesOSSAdapter()
{
	return static_cast<FOnlineServicesOSSAdapter&>(Services);
}

const IOnlineSubsystem& FCommerceOSSAdapter::GetSubsystem() const
{
	return GetOnlineServicesOSSAdapter().GetSubsystem();
}

IOnlinePurchasePtr FCommerceOSSAdapter::GetPurchaseInterface() const
{
	return GetSubsystem().GetPurchaseInterface();
}

IOnlineStoreV2Ptr FCommerceOSSAdapter::GetStoreInterface() const
{
	return GetSubsystem().GetStoreV2Interface();
}

IOnlineExternalUIPtr FCommerceOSSAdapter::GetExternalUIInterface() const
{
	return GetSubsystem().GetExternalUIInterface();
}

} // namspace UE::Online