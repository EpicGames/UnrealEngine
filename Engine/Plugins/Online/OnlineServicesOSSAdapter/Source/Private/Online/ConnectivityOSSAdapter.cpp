// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/ConnectivityOSSAdapter.h"

#include "Online/OnlineServicesOSSAdapter.h"

#include "OnlineSubsystem.h"

namespace UE::Online {

void FConnectivityOSSAdapter::PostInitialize()
{
	Super::PostInitialize();

	IOnlineSubsystem& Subsystem = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem();
	OnConnectionStatusChangedHandle = Subsystem.OnConnectionStatusChangedDelegates.AddLambda(
		[WeakThis = TWeakPtr<IConnectivity>(AsShared()), this](const FString& ServiceName, EOnlineServerConnectionStatus::Type LastConnectionStatus, EOnlineServerConnectionStatus::Type NewConnectionStatus)
		{
			TSharedPtr<IConnectivity> PinnedThis = WeakThis.Pin();
			if (PinnedThis.IsValid())
			{
				FConnectionStatusChanged Params;
				Params.ServiceName = ServiceName;

				switch (LastConnectionStatus)
				{
					case EOnlineServerConnectionStatus::Normal:
					case EOnlineServerConnectionStatus::Connected:
						Params.PreviousStatus = EOnlineServicesConnectionStatus::Connected;
						break;

					case EOnlineServerConnectionStatus::NotConnected:
					case EOnlineServerConnectionStatus::ConnectionDropped:
					case EOnlineServerConnectionStatus::NoNetworkConnection:
					case EOnlineServerConnectionStatus::ServiceUnavailable:
					case EOnlineServerConnectionStatus::UpdateRequired:
					case EOnlineServerConnectionStatus::ServersTooBusy:
					case EOnlineServerConnectionStatus::DuplicateLoginDetected:
					case EOnlineServerConnectionStatus::InvalidUser:
					case EOnlineServerConnectionStatus::NotAuthorized:
					case EOnlineServerConnectionStatus::InvalidSession:
						Params.PreviousStatus = EOnlineServicesConnectionStatus::NotConnected;
						break;
				}

				switch (NewConnectionStatus)
				{
					case EOnlineServerConnectionStatus::Normal:
					case EOnlineServerConnectionStatus::Connected:
						Params.CurrentStatus = EOnlineServicesConnectionStatus::Connected;
						break;

					case EOnlineServerConnectionStatus::NotConnected:
					case EOnlineServerConnectionStatus::ConnectionDropped:
					case EOnlineServerConnectionStatus::NoNetworkConnection:
					case EOnlineServerConnectionStatus::ServiceUnavailable:
					case EOnlineServerConnectionStatus::UpdateRequired:
					case EOnlineServerConnectionStatus::ServersTooBusy:
					case EOnlineServerConnectionStatus::DuplicateLoginDetected:
					case EOnlineServerConnectionStatus::InvalidUser:
					case EOnlineServerConnectionStatus::NotAuthorized:
					case EOnlineServerConnectionStatus::InvalidSession:
						Params.CurrentStatus = EOnlineServicesConnectionStatus::NotConnected;
						break;
				}

				CurrentStatus.Add(ServiceName, Params.CurrentStatus);

				OnConnectionStatusChangedEvent.Broadcast(Params);
			}
		});
}

void FConnectivityOSSAdapter::PreShutdown()
{
	IOnlineSubsystem& Subsystem = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem();
	if (OnConnectionStatusChangedHandle.IsValid())
	{
		Subsystem.OnConnectionStatusChangedDelegates.Remove(OnConnectionStatusChangedHandle);
		OnConnectionStatusChangedHandle.Reset();
	}

	Super::PreShutdown();
}

TOnlineResult<FGetConnectionStatus> FConnectivityOSSAdapter::GetConnectionStatus(FGetConnectionStatus::Params&& Params)
{
	FGetConnectionStatus::Result Result;
	Result.Status = CurrentStatus.FindOrAdd(Params.ServiceName, EOnlineServicesConnectionStatus::NotConnected);

	return TOnlineResult<FGetConnectionStatus>(Result);
}

/* UE::Online*/ }
