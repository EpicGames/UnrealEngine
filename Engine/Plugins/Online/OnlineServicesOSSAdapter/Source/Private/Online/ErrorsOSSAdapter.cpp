// Copyright Epic Games, Inc. All Rights Reserved.
#include "Online/ErrorsOSSAdapter.h"
#include "Online/OnlineErrorDefinitions.h"
#include "OnlineSubsystemTypes.h"
#include "OnlineError.h"

namespace UE::Online::Errors {

TArray<TOssPlatformErrorHandler> PlatformFunctions;
void AddOssPlatformErrorHandler(TOssPlatformErrorHandler&& InFunction)
{
	PlatformFunctions.Add(MoveTemp(InFunction));
}


inline FOnlineError Internal_OssGetCommonError(FOnlineError Error, EOnlineErrorResult Result)
{
	switch (Result)
	{
	case EOnlineErrorResult::Success: return Errors::Success(Error);
	case EOnlineErrorResult::NoConnection: return Errors::NoConnection(Error);
	case EOnlineErrorResult::RequestFailure: return Errors::RequestFailure(Error);
	case EOnlineErrorResult::InvalidCreds: return Errors::InvalidCreds(Error);
	case EOnlineErrorResult::InvalidUser: return Errors::InvalidUser(Error);
	case EOnlineErrorResult::InvalidAuth: return Errors::InvalidAuth(Error);
	case EOnlineErrorResult::AccessDenied: return Errors::AccessDenied(Error);
	case EOnlineErrorResult::TooManyRequests: return Errors::TooManyRequests(Error);
	case EOnlineErrorResult::AlreadyPending: return Errors::AlreadyPending(Error);
	case EOnlineErrorResult::InvalidParams: return Errors::InvalidParams(Error);
	case EOnlineErrorResult::CantParse: return Errors::CantParse(Error);
	case EOnlineErrorResult::InvalidResults: return Errors::InvalidResults(Error);
	case EOnlineErrorResult::IncompatibleVersion: return Errors::IncompatibleVersion(Error);
	case EOnlineErrorResult::NotConfigured: return Errors::NotConfigured(Error);
	case EOnlineErrorResult::NotImplemented: return Errors::NotImplemented(Error);
	case EOnlineErrorResult::MissingInterface: return Errors::MissingInterface(Error);
	case EOnlineErrorResult::Canceled: return Errors::Cancelled(Error);
	default: return Error;
	}
}

inline FOnlineError Internal_OssWrapInner(FOnlineError Error, const FOnlineErrorOss& Result)
{
	if (Result.WasSuccessful())
	{
		return Errors::Success(Error);
	}

	for (const TOssPlatformErrorHandler& PlatformHandler : PlatformFunctions)
	{
		TOptional<FOnlineError> NewError = PlatformHandler(Error, Result);
		if (NewError.IsSet())
		{
			return Internal_OssGetCommonError(NewError.GetValue(), Result.GetErrorResult());
		}
	}

	return Internal_OssGetCommonError(Error, Result.GetErrorResult());
}

FOnlineError FromOssError(const FOnlineErrorOss& Result)
{
	FString ErrorCode;
	FText ErrorMessage = Result.GetErrorMessage();

	if (Result.GetErrorRaw().Len() > 0)
	{
		ErrorCode = FString::Printf(TEXT("{code: %s, raw: %s}"), *Result.GetErrorCode(), *Result.GetErrorRaw());
	}
	else
	{
		ErrorCode = Result.GetErrorCode();
	}
	
	FString FriendlyErrorCode = Result.GetErrorCode();

	return Internal_OssWrapInner(FOnlineError(ErrorCode::Create(ErrorCode::Category::Oss_System, ErrorCode::Category::Oss, (uint32)Result.GetErrorResult()), MakeShared<FOnlineErrorDetails, ESPMode::ThreadSafe>(MoveTemp(FriendlyErrorCode), MoveTemp(ErrorCode), MoveTemp(ErrorMessage)), nullptr), Result);
}

FOnlineError FromOssErrorCode(const FString& ErrorCode)
{
	EOnlineErrorResult ErrorResult = EOnlineErrorResult::FailExtended;
	if (ErrorCode.EndsWith(TEXT("no_connection")))
	{
		ErrorResult = EOnlineErrorResult::NoConnection;
	}
	else if (ErrorCode.EndsWith(TEXT("request_failure")))
	{
		ErrorResult = EOnlineErrorResult::RequestFailure;
	}
	else if (ErrorCode.EndsWith(TEXT("invalid_creds")))
	{
		ErrorResult = EOnlineErrorResult::InvalidCreds;
	}
	else if (ErrorCode.EndsWith(TEXT("invalid_user")))
	{
		ErrorResult = EOnlineErrorResult::InvalidUser;
	}
	else if (ErrorCode.EndsWith(TEXT("invalid_auth")))
	{
		ErrorResult = EOnlineErrorResult::InvalidAuth;
	}
	else if (ErrorCode.EndsWith(TEXT("access_denied")))
	{
		ErrorResult = EOnlineErrorResult::AccessDenied;
	}
	else if (ErrorCode.EndsWith(TEXT("too_many_requests")))
	{
		ErrorResult = EOnlineErrorResult::TooManyRequests;
	}
	else if (ErrorCode.EndsWith(TEXT("already_pending")))
	{
		ErrorResult = EOnlineErrorResult::AlreadyPending;
	}
	else if (ErrorCode.EndsWith(TEXT("invalid_params")))
	{
		ErrorResult = EOnlineErrorResult::InvalidParams;
	}
	else if (ErrorCode.EndsWith(TEXT("cant_parse")))
	{
		ErrorResult = EOnlineErrorResult::CantParse;
	}
	else if (ErrorCode.EndsWith(TEXT("invalid_results")))
	{
		ErrorResult = EOnlineErrorResult::InvalidResults;
	}
	else if (ErrorCode.EndsWith(TEXT("incompatible_version")))
	{
		ErrorResult = EOnlineErrorResult::IncompatibleVersion;
	}
	else if (ErrorCode.EndsWith(TEXT("not_configured")))
	{
		ErrorResult = EOnlineErrorResult::NotConfigured;
	}
	else if (ErrorCode.EndsWith(TEXT("not_implemented")))
	{
		ErrorResult = EOnlineErrorResult::NotImplemented;
	}
	else if (ErrorCode.EndsWith(TEXT("missing_interface")))
	{
		ErrorResult = EOnlineErrorResult::MissingInterface;
	}
	else if (ErrorCode.EndsWith(TEXT("canceled")))
	{
		ErrorResult = EOnlineErrorResult::Canceled;
	}
	else if (ErrorCode.EndsWith(TEXT("fail_extended")))
	{
		ErrorResult = EOnlineErrorResult::FailExtended;
	}

	// construct an OSS FOnlineError from the result
	::FOnlineError Result = ::FOnlineError::CreateError(FString(), ErrorResult, ErrorCode);

	return Internal_OssWrapInner(FOnlineError(ErrorCode::Create(ErrorCode::Category::Oss_System, ErrorCode::Category::Oss, (uint32)Result.GetErrorResult()), 
		MakeShared<FOnlineErrorDetails, ESPMode::ThreadSafe>(CopyTemp(ErrorCode), CopyTemp(ErrorCode), CopyTemp(Result.GetErrorMessage())), nullptr), Result);
}

} //namespace UE::Online::Errors

ONLINESERVICESOSSADAPTER_API bool operator==(const UE::Online::FOnlineError& Left, const FOnlineErrorOss& Right)
{
	return Left == UE::Online::Errors::FromOssError(Right);
}

ONLINESERVICESOSSADAPTER_API bool operator==(const FOnlineErrorOss& Left, const UE::Online::FOnlineError& Right)
{
	return Right == UE::Online::Errors::FromOssError(Left);
}

ONLINESERVICESOSSADAPTER_API bool operator!=(const UE::Online::FOnlineError& Left, const FOnlineErrorOss& Right)
{
	return !(Left == UE::Online::Errors::FromOssError(Right));
}

ONLINESERVICESOSSADAPTER_API bool operator!=(const FOnlineErrorOss& Left, const UE::Online::FOnlineError& Right)
{
	return !(Right == UE::Online::Errors::FromOssError(Left));
}