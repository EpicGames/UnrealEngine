// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/ExternalUIOSSAdapter.h"

#include "Online/OnlineServicesOSSAdapter.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/OnlineIdOSSAdapter.h"

#include "OnlineSubsystem.h"
#include "Interfaces/OnlineExternalUIInterface.h"

namespace UE::Online {

void FExternalUIOSSAdapter::PostInitialize()
{
	Super::PostInitialize();

	OnExternalUIChangeHandle = GetExternalUIInterface()->OnExternalUIChangeDelegates.AddLambda(
		[WeakThis = TWeakPtr<IExternalUI>(AsShared()), this](bool bIsOpening)
		{
			TSharedPtr<IExternalUI> PinnedThis = WeakThis.Pin();
			if (PinnedThis.IsValid())
			{
				FExternalUIStatusChanged StatusChanged;
				StatusChanged.bIsOpening = bIsOpening;

				OnExternalUIStatusChangedEvent.Broadcast(StatusChanged);
			}
		});
}

void FExternalUIOSSAdapter::PreShutdown()
{
	if (OnExternalUIChangeHandle.IsValid())
	{
		GetExternalUIInterface()->OnExternalUIChangeDelegates.Remove(OnExternalUIChangeHandle);
		OnExternalUIChangeHandle.Reset();
	}

	Super::PreShutdown();
}

TOnlineAsyncOpHandle<FExternalUIShowLoginUI> FExternalUIOSSAdapter::ShowLoginUI(FExternalUIShowLoginUI::Params&& Params)
{
	TSharedRef<TOnlineAsyncOp<FExternalUIShowLoginUI>> Op = GetOp<FExternalUIShowLoginUI>(MoveTemp(Params));

	if (!Op->IsReady())
	{
		int32 LocalUserIndex = Op->GetParams().PlatformUserId.GetInternalId();

		if (LocalUserIndex < 0 || LocalUserIndex >= MAX_LOCAL_PLAYERS)
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this, LocalUserIndex](TOnlineAsyncOp<FExternalUIShowLoginUI>& Op)
			{
				TSharedPtr<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				bool bShowedUI = GetExternalUIInterface()->ShowLoginUI(LocalUserIndex, true, false,
					FOnLoginUIClosedDelegate::CreateLambda(
						[this, WeakOp = Op.AsWeak(), Promise](FUniqueNetIdPtr UniqueId, const int ControllerIndex, const ::FOnlineError& Error)
						{
							TSharedPtr<TOnlineAsyncOp<FExternalUIShowLoginUI>> PinnedOp = WeakOp.Pin();
							if (PinnedOp.IsValid())
							{
								if (Error.WasSuccessful())
								{
									FAccountId AccountId = static_cast<FOnlineServicesOSSAdapter&>(Services).GetAccountIdRegistry().FindOrAddHandle(UniqueId.ToSharedRef());
									FExternalUIShowLoginUI::Result Result = { MakeShared<FAccountInfo>() };
									Result.AccountInfo->PlatformUserId = PinnedOp->GetParams().PlatformUserId;
									Result.AccountInfo->AccountId = AccountId;
									Result.AccountInfo->LoginStatus = ELoginStatus::LoggedIn;

									PinnedOp->SetResult(MoveTemp(Result));
								}
								else
								{
									PinnedOp->SetError(Errors::Unknown()); // TODO: Translate v1 to v2 error
								}
								
								Promise->SetValue();
							}
						}
					));

				if (!bShowedUI)
				{
					Op.SetError(Errors::Unknown());
				}

				return Promise->GetFuture();
			})
			.Enqueue(GetSerialQueue());
	}

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FExternalUIShowFriendsUI> FExternalUIOSSAdapter::ShowFriendsUI(FExternalUIShowFriendsUI::Params&& Params)
{
	TSharedRef<TOnlineAsyncOp<FExternalUIShowFriendsUI>> Op = GetOp<FExternalUIShowFriendsUI>(MoveTemp(Params));

	if (!Op->IsReady())
	{
		int32 LocalUserIndex = Services.Get<FAuthOSSAdapter>()->GetLocalUserNum(Op->GetParams().LocalAccountId);

		if (LocalUserIndex < 0 || LocalUserIndex >= MAX_LOCAL_PLAYERS)
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this, LocalUserIndex](TOnlineAsyncOp<FExternalUIShowFriendsUI>& Op)
			{
				if (GetExternalUIInterface()->ShowFriendsUI(LocalUserIndex))
				{
					Op.SetResult(FExternalUIShowFriendsUI::Result());
				}
				else
				{
					Op.SetError(Errors::Unknown());
				}
			})
			.Enqueue(GetSerialQueue());
	}

	return Op->GetHandle();
}

IOnlineExternalUIPtr FExternalUIOSSAdapter::GetExternalUIInterface() const
{
	return static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem().GetExternalUIInterface();
}

/* UE::Online */ }
