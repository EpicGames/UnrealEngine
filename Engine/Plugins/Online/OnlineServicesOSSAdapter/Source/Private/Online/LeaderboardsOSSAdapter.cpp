// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/LeaderboardsOSSAdapter.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/DelegateAdapter.h"
#include "Online/ErrorsOSSAdapter.h"
#include "Online/OnlineServicesOSSAdapter.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/OnlineErrorDefinitions.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"

namespace UE::Online {

void FLeaderboardsOSSAdapter::PostInitialize()
{
	Super::PostInitialize();

	Auth = Services.Get<FAuthOSSAdapter>();
	LeaderboardsInterface = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem().GetLeaderboardsInterface();
	check(LeaderboardsInterface);
}

TOnlineAsyncOpHandle<FReadEntriesForUsers> FLeaderboardsOSSAdapter::ReadEntriesForUsers(FReadEntriesForUsers::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesForUsers> Op = GetOp<FReadEntriesForUsers>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FReadEntriesForUsers>& Op)
	{
		const FUniqueNetIdPtr LocalUniqueNetId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);

		if (!LocalUniqueNetId)
		{
			Op.SetError(Errors::InvalidUser());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		if (Op.GetParams().AccountIds.IsEmpty() || Op.GetParams().BoardName.IsEmpty())
		{
			Op.SetError(Errors::InvalidParams());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TArray<FUniqueNetIdRef> NetIds;
		for (const FAccountId& AccountId : Op.GetParams().AccountIds)
		{
			const FUniqueNetIdPtr UniqueNetId = Auth->GetUniqueNetId(AccountId);
			if (!UniqueNetId)
			{
				UE_LOGF(LogOnlineServices, Warning, "Failed to get unique net id for user %ls", *ToLogString(AccountId));
				Op.SetError(Errors::InvalidUser());
				return MakeFulfilledPromise<void>().GetFuture();
			}

			NetIds.Add(UniqueNetId.ToSharedRef());
		}

		if (TOptional<FOnlineError> OnlineError = PrepareLeaderboardReadObject(Op.GetParams().BoardName))
		{
			Op.SetError(MoveTemp(OnlineError.GetValue()));
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, LeaderboardsInterface->OnLeaderboardReadCompleteDelegates,
			[this, WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise)](bool bSuccess) mutable
		{
			if (TOnlineAsyncOpPtr<FReadEntriesForUsers> Op = WeakOp.Pin())
			{
				if (!bSuccess)
				{
					ReadObject.Reset();
					Op->SetError(Errors::Unknown());
				}
				else
				{
					FReadEntriesForUsers::Result Result;
					ReadLeaderboardsResultFromV1ToV2(Op->GetParams().BoardName, Result.Entries);
					Op->SetResult(MoveTemp(Result));
				}
			}
			Promise.SetValue();
		});

		FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();
		if (!LeaderboardsInterface->ReadLeaderboards(NetIds, ReadObjectRef))
		{
			Op.SetError(Errors::Unknown());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		return Future;
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FReadEntriesAroundRank> FLeaderboardsOSSAdapter::ReadEntriesAroundRank(FReadEntriesAroundRank::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesAroundRank> Op = GetOp<FReadEntriesAroundRank>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FReadEntriesAroundRank>& Op)
	{
		const FUniqueNetIdPtr LocalUserId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		if (Op.GetParams().Limit == 0 || Op.GetParams().BoardName.IsEmpty())
		{
			Op.SetError(Errors::InvalidParams());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		if (TOptional<FOnlineError> OnlineError = PrepareLeaderboardReadObject(Op.GetParams().BoardName))
		{
			Op.SetError(MoveTemp(OnlineError.GetValue()));
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, LeaderboardsInterface->OnLeaderboardReadCompleteDelegates,
			[this, WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise)](bool bSuccess) mutable
		{
			if (TOnlineAsyncOpPtr<FReadEntriesAroundRank> Op = WeakOp.Pin())
			{
				if (!bSuccess)
				{
					ReadObject.Reset();
					Op->SetError(Errors::Unknown());
				}
				else
				{
					FReadEntriesAroundRank::Result Result;
					ReadLeaderboardsResultFromV1ToV2(Op->GetParams().BoardName, Result.Entries);
					Op->SetResult(MoveTemp(Result));
				}
			}
			Promise.SetValue();
		});

		FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();
		if (!LeaderboardsInterface->ReadLeaderboardsAroundRank(Op.GetParams().Rank, Op.GetParams().Limit, ReadObjectRef))
		{
			Op.SetError(Errors::Unknown());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		return Future;
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FReadEntriesAroundUser> FLeaderboardsOSSAdapter::ReadEntriesAroundUser(FReadEntriesAroundUser::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesAroundUser> Op = GetOp<FReadEntriesAroundUser>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FReadEntriesAroundUser>& Op)
	{
		const FUniqueNetIdPtr LocalUserId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		const FUniqueNetIdPtr TargetUniqueNetId = Auth->GetUniqueNetId(Op.GetParams().AccountId);
		if (!TargetUniqueNetId)
		{
			Op.SetError(Errors::InvalidUser());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		if (Op.GetParams().Limit == 0 || Op.GetParams().BoardName.IsEmpty())
		{
			Op.SetError(Errors::InvalidParams());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		if (TOptional<FOnlineError> OnlineError = PrepareLeaderboardReadObject(Op.GetParams().BoardName))
		{
			Op.SetError(MoveTemp(OnlineError.GetValue()));
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, LeaderboardsInterface->OnLeaderboardReadCompleteDelegates,
			[this, WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise)](bool bSuccess) mutable
		{
			if (TOnlineAsyncOpPtr<FReadEntriesAroundUser> Op = WeakOp.Pin())
			{
				if (!bSuccess)
				{
					ReadObject.Reset();
					Op->SetError(Errors::Unknown());
				}
				else
				{
					FReadEntriesAroundUser::Result Result;
					ReadLeaderboardsResultFromV1ToV2(Op->GetParams().BoardName, Result.Entries);
					Op->SetResult(MoveTemp(Result));
				}
			}
			Promise.SetValue();
		});

		FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();
		if (!LeaderboardsInterface->ReadLeaderboardsAroundUser(TargetUniqueNetId.ToSharedRef(), Op.GetParams().Limit, ReadObjectRef))
		{
			Op.SetError(Errors::Unknown());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		return Future;
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOptional<FOnlineError> FLeaderboardsOSSAdapter::PrepareLeaderboardReadObject(const FString& BoardName)
{
	TOptional<FOnlineError> OnlineError;

	if (ReadObject)
	{
		OnlineError = Errors::AlreadyPending();
	}
	else
	{
		ReadObject = MakeShareable(new FOnlineLeaderboardRead());
		ReadObject->SortedColumn = BoardName;
		ReadObject->LeaderboardName = BoardName;
	}

	return OnlineError;
}

void FLeaderboardsOSSAdapter::ReadLeaderboardsResultFromV1ToV2(const FString& BoardName, TArray<FLeaderboardEntry>& OutEntries)
{
	for (const FOnlineStatsRow& OnlineStatsRow : ReadObject->Rows)
	{
		const FVariantData* VariantData = OnlineStatsRow.Columns.Find(*BoardName);
		FLeaderboardEntry& LeaderboardEntry = OutEntries.Emplace_GetRef();
		LeaderboardEntry.AccountId = Auth->GetAccountId(OnlineStatsRow.PlayerId.ToSharedRef());
		LeaderboardEntry.Rank = OnlineStatsRow.Rank;
		VariantData->GetValue(LeaderboardEntry.Score);
	}

	ReadObject.Reset();
}

/* UE::Online */ }
