// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesOSSAdapter.h"

#include "Online/AchievementsOSSAdapter.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/ConnectivityOSSAdapter.h"
#include "Online/ExternalUIOSSAdapter.h"
#include "Online/LeaderboardsOSSAdapter.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/PresenceOSSAdapter.h"
#include "Online/PrivilegesOSSAdapter.h"
#include "Online/SessionsOSSAdapter.h"
#include "Online/SocialOSSAdapter.h"
#include "Online/StatsOSSAdapter.h"
#include "Online/TitleFileOSSAdapter.h"
#include "Online/UserFileOSSAdapter.h"
#include "Online/UserInfoOSSAdapter.h"
#include "Online/CommerceOSSAdapter.h"

#include "OnlineSubsystem.h"
#include "Online/OnlineServicesOSSAdapterModule.h"
#include "Online/OnlineServicesRegistry.h"

namespace UE::Online {

FOnlineServicesOSSAdapter::FOnlineServicesOSSAdapter(EOnlineServices InServicesType, const FString& InServiceConfigName, FName InInstanceName, IOnlineSubsystem* InSubsystem)
	: FOnlineServicesCommon(InServiceConfigName, InInstanceName, NAME_None)
	, ServicesType(InServicesType)
	, Subsystem(InSubsystem)
{
}

void FOnlineServicesOSSAdapter::RegisterComponents()
{
	Components.Register<FAuthOSSAdapter>(*this);
	Components.Register<FConnectivityOSSAdapter>(*this);

	if (Subsystem->GetLeaderboardsInterface().IsValid())
	{
		Components.Register<FLeaderboardsOSSAdapter>(*this);
	}

	Components.Register<FPresenceOSSAdapter>(*this);
	Components.Register<FPrivilegesOSSAdapter>(*this);

	if (Subsystem->GetStatsInterface().IsValid())
	{
		Components.Register<FStatsOSSAdapter>(*this);
	}

	if (Subsystem->GetFriendsInterface().IsValid())
	{
		Components.Register<FSocialOSSAdapter>(*this);
	}

	if (Subsystem->GetSessionInterface().IsValid())
	{
		Components.Register<FSessionsOSSAdapter>(*this);
	}
	if (Subsystem->GetAchievementsInterface().IsValid())
	{
		Components.Register<FAchievementsOSSAdapter>(*this);
	}
	if (Subsystem->GetExternalUIInterface().IsValid())
	{
		Components.Register<FExternalUIOSSAdapter>(*this);
	}
	if (Subsystem->GetTitleFileInterface().IsValid())
	{
		Components.Register<FTitleFileOSSAdapter>(*this);
	}
	if (Subsystem->GetUserCloudInterface().IsValid())
	{
		Components.Register<FUserFileOSSAdapter>(*this);
	}
	if (Subsystem->GetUserInterface().IsValid())
	{
		Components.Register<FUserInfoOSSAdapter>(*this);
	}
	if (Subsystem->GetPurchaseInterface().IsValid() && Subsystem->GetStoreV2Interface().IsValid())
	{
		Components.Register<FCommerceOSSAdapter>(*this);
	}

	FOnlineServicesCommon::RegisterComponents();
}

void FOnlineServicesOSSAdapter::Initialize()
{
	AccountIdRegistry = static_cast<FOnlineAccountIdRegistryOSSAdapter*>(FOnlineIdRegistryRegistry::Get().GetAccountIdRegistry(ServicesType));
	
	if (FOnlineServicesOSSAdapterModule* OSSAdapterModule = FModuleManager::GetModulePtr<FOnlineServicesOSSAdapterModule>("OnlineServicesOSSAdapter"))
	{
		OSSAdapterModule->RegisterOSSAdapter(StaticCastSharedPtr<FOnlineServicesOSSAdapter>(AsShared().ToSharedPtr()));
	} 
	
	FOnlineServicesCommon::Initialize();
}

TOnlineResult<FGetResolvedConnectString> FOnlineServicesOSSAdapter::GetResolvedConnectString(FGetResolvedConnectString::Params&& Params)
{
	FSessionsOSSAdapter* SessionsOSSAdapter = Get<FSessionsOSSAdapter>();
	check(SessionsOSSAdapter);

	return SessionsOSSAdapter->GetResolvedConnectString(Params);
}

/* UE::Online*/ }
