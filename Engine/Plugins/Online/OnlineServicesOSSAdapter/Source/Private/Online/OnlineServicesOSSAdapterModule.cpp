// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesOSSAdapterModule.h"

#include "CoreMinimal.h"
#include "OnlineDelegates.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "Online/OnlineServicesOSSAdapter.h"
#include "Online/OnlineServicesRegistry.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/SessionsOSSAdapter.h"

#include "OnlineSubsystem.h"

namespace UE::Online
{

struct FOSSAdapterService
{
	EOnlineServices Service = EOnlineServices::Default;
	FString ConfigName;
	FName OnlineSubsystem;
	int32 Priority = -1;
};

struct FOSSAdapterConfig
{
	TArray<FOSSAdapterService> Services;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FOSSAdapterService)
	ONLINE_STRUCT_FIELD(FOSSAdapterService, Service),
	ONLINE_STRUCT_FIELD(FOSSAdapterService, ConfigName),
	ONLINE_STRUCT_FIELD(FOSSAdapterService, OnlineSubsystem),
	ONLINE_STRUCT_FIELD(FOSSAdapterService, Priority)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FOSSAdapterConfig)
	ONLINE_STRUCT_FIELD(FOSSAdapterConfig, Services)
END_ONLINE_STRUCT_META()

/* Meta */ }

class FOnlineServicesFactoryOSSAdapter : public IOnlineServicesFactory
{
public:
	FOnlineServicesFactoryOSSAdapter(const FOSSAdapterService& InConfig)
		: Config(InConfig)
	{
	}

	virtual ~FOnlineServicesFactoryOSSAdapter() {}
	virtual TSharedPtr<IOnlineServices> Create(FName InInstanceName, FName /*InstanceConfigName*/) override
	{
		FName CombinedInstanceName = InInstanceName.IsNone() ? (/*NoSuffix*/Config.OnlineSubsystem) : FName(*FString::Printf(TEXT("%s:%s"), *Config.OnlineSubsystem.ToString(), *InInstanceName.ToString()));
		IOnlineSubsystem* Subsystem = IOnlineSubsystem::Get(CombinedInstanceName);
		if (Subsystem != nullptr)
		{
			return MakeShared<FOnlineServicesOSSAdapter>(Config.Service, Config.ConfigName, InInstanceName, Subsystem);
		}
		else
		{
			return nullptr;
		}
	}
protected:
	FOSSAdapterService Config;
};

void FOnlineServicesOSSAdapterModule::StartupModule()
{
	FOnlineConfigProviderGConfig ConfigProvider(GEngineIni);
	FOSSAdapterConfig Config;
	if (LoadConfig(ConfigProvider, TEXT("OnlineServices.OSSAdapter"), Config))
	{
		CachedServices = Config.Services;
		FModuleManager& ModuleManager = FModuleManager::Get();
		if (!CachedServices.IsEmpty())
		{
			ModuleManager.LoadModule(TEXT("OnlineSubsystem"));
		}

		for (const FOSSAdapterService& ServiceConfig : CachedServices)
		{
			if (IOnlineSubsystem::IsEnabled(ServiceConfig.OnlineSubsystem))
			{
				const FName SubsystemName(FString::Printf(TEXT("OnlineSubsystem%s"), *ServiceConfig.OnlineSubsystem.ToString()));
				if (!ModuleManager.IsModuleLoaded(SubsystemName))
				{
					ModuleManager.LoadModule(SubsystemName);
				}
				FOnlineServicesRegistry::Get().RegisterServicesFactory(ServiceConfig.Service, MakeUnique<FOnlineServicesFactoryOSSAdapter>(ServiceConfig), ServiceConfig.Priority);
				FOnlineIdRegistryRegistry::Get().RegisterAccountIdRegistry(ServiceConfig.Service, new FOnlineAccountIdRegistryOSSAdapter(ServiceConfig.Service), ServiceConfig.Priority);
				FOnlineIdRegistryRegistry::Get().RegisterSessionIdRegistry(ServiceConfig.Service, new FOnlineSessionIdRegistryOSSAdapter(ServiceConfig.Service), ServiceConfig.Priority);
				FOnlineIdRegistryRegistry::Get().RegisterSessionInviteIdRegistry(ServiceConfig.Service, new FOnlineSessionInviteIdRegistryOSSAdapter(ServiceConfig.Service), ServiceConfig.Priority);
			}
		}
	}
	
	OnSubsystemPreReloadHandle = FOnlineSubsystemDelegates::OnDefaultOnlineSubsystemPreReloaded.AddRaw(this, &FOnlineServicesOSSAdapterModule::OnSubsystemPreReload);
}

void FOnlineServicesOSSAdapterModule::ShutdownModule()
{
	for (FOSSAdapterService& ServiceConfig : CachedServices)
	{
		FOnlineServicesRegistry::Get().UnregisterServicesFactory(ServiceConfig.Service, ServiceConfig.Priority);
		FOnlineIdRegistryRegistry::Get().UnregisterAccountIdRegistry(ServiceConfig.Service, ServiceConfig.Priority);
		FOnlineIdRegistryRegistry::Get().UnregisterSessionIdRegistry(ServiceConfig.Service, ServiceConfig.Priority);
		FOnlineIdRegistryRegistry::Get().UnregisterSessionInviteIdRegistry(ServiceConfig.Service, ServiceConfig.Priority);
	}
	CachedServices.Empty();
	FOnlineSubsystemDelegates::OnDefaultOnlineSubsystemPreReloaded.Remove(OnSubsystemPreReloadHandle);
	OnSubsystemPreReloadHandle.Reset();
}
	
void FOnlineServicesOSSAdapterModule::GetAllAdapters(TArray<TSharedPtr<FOnlineServicesOSSAdapter>>& OutOSSAdapters)
{
	for (int32 AdapterIdx = OSSAdapters.Num() - 1; AdapterIdx >= 0; AdapterIdx--)
	{
		if (TSharedPtr<FOnlineServicesOSSAdapter> OSSAdapter = OSSAdapters[AdapterIdx].Pin())
		{
			OutOSSAdapters.Add(OSSAdapter);
		}
		else
		{
			OSSAdapters.RemoveAtSwap(AdapterIdx);
		}
	}
}
	
void FOnlineServicesOSSAdapterModule::RegisterOSSAdapter(const TSharedPtr<FOnlineServicesOSSAdapter> OnlineServicesOSSAdapter)
{
	OSSAdapters.Add(OnlineServicesOSSAdapter);
}
	
void FOnlineServicesOSSAdapterModule::OnSubsystemPreReload(IOnlineSubsystem* OnlineSubsystem)
{
	TArray<TSharedPtr<FOnlineServicesOSSAdapter>> OutOnlineServices;
	GetAllAdapters(OutOnlineServices);
	
	for (const TSharedPtr<FOnlineServicesOSSAdapter>& OnlineServiceOSSAdapter : OutOnlineServices)
	{
		if (&OnlineServiceOSSAdapter->GetSubsystem() == OnlineSubsystem)
		{
			DestroyService(OnlineServiceOSSAdapter->GetServicesProvider(), OnlineServiceOSSAdapter->GetInstanceName(),
				OnlineServiceOSSAdapter->GetInstanceConfigName());
		}
	}
}

/* UE::Online */ }

IMPLEMENT_MODULE(UE::Online::FOnlineServicesOSSAdapterModule, OnlineServicesOSSAdapter);
