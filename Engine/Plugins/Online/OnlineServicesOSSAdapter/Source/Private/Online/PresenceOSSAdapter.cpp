// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/PresenceOSSAdapter.h"
#include "Online/AuthOSSAdapter.h"

#include "Online/OnlineServicesOSSAdapter.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/DelegateAdapter.h"

#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"


namespace UE::Online {

EUserPresenceStatus FPresenceOSSAdapter::StatusV1toV2(EOnlinePresenceState::Type Status)
{
	switch (Status)
	{
	case EOnlinePresenceState::Online:
		return EUserPresenceStatus::Online;
	case EOnlinePresenceState::Offline:
		return EUserPresenceStatus::Offline;
	case EOnlinePresenceState::Away:
		return EUserPresenceStatus::Away;
	case EOnlinePresenceState::ExtendedAway:
		return EUserPresenceStatus::ExtendedAway;
	case EOnlinePresenceState::DoNotDisturb:
		return EUserPresenceStatus::DoNotDisturb;
	default:
		return EUserPresenceStatus::Unknown;
	}
}

EOnlinePresenceState::Type FPresenceOSSAdapter::StatusV2toV1(EUserPresenceStatus Status)
{
	switch (Status)
	{
	case EUserPresenceStatus::Offline:
		return EOnlinePresenceState::Offline;
	case EUserPresenceStatus::Online:
		return EOnlinePresenceState::Online;
	case EUserPresenceStatus::Away:
		return EOnlinePresenceState::Away;
	case EUserPresenceStatus::ExtendedAway:
		return EOnlinePresenceState::ExtendedAway;
	case EUserPresenceStatus::DoNotDisturb:
		return EOnlinePresenceState::DoNotDisturb;
	default:
		return EOnlinePresenceState::Offline; // no default/unknown value for V1
	}
}

EUserPresenceGameStatus GameStatusV1toV2(FOnlineUserPresence& Presence)
{
	if (Presence.bIsPlayingThisGame)
	{
		return EUserPresenceGameStatus::PlayingThisGame;
	}
	else if (Presence.bIsPlaying)
	{
		return EUserPresenceGameStatus::PlayingOtherGame;
	}
	else 
	{
		return EUserPresenceGameStatus::Unknown;
	}
}

void PropertiesV1toV2(FOnlineKeyValuePairs<FPresenceKey, FVariantData>& PropertiesIn, TMap<FString, FPresenceProperty>& PropertiesOut)
{
	for (const TPair<FPresenceKey, FVariantData>& Pair : PropertiesIn)
	{
		FPresenceProperty Value;

		Value.Set<FString>(Pair.Value.ToString()); // TODO: Proper conversion from V2 Variants to V1

		PropertiesOut.Add(Pair.Key, Value);
	}
}

void PropertiesV2toV1(TMap<FString, FPresenceProperty> PropertiesIn, FOnlineKeyValuePairs<FPresenceKey, FVariantData>& PropertiesOut)
{
	for (const TPair<FString, FPresenceProperty>& Pair : PropertiesIn)
	{
		FVariantData Data;
		Data.SetValue(Pair.Value.Get<FString>()); // TODO: Proper conversion from V2 Variants to V1
		PropertiesOut.Add(Pair.Key, Data);
	}
}

TSharedRef<FUserPresence> FPresenceOSSAdapter::PresenceV1toV2(FAccountId AccountId, FOnlineUserPresence& Presence)
{
	TSharedRef<FUserPresence> PresenceV2 = MakeShared<FUserPresence>();

	PresenceV2->AccountId = AccountId;
	PresenceV2->Status = StatusV1toV2(Presence.Status.State);
	PresenceV2->GameStatus = GameStatusV1toV2(Presence);
	PresenceV2->Joinability = (Presence.bIsJoinable) ? EUserPresenceJoinability::Public : EUserPresenceJoinability::Private;
	PresenceV2->StatusString = Presence.Status.StatusStr;
	PresenceV2->RichPresenceString = Presence.Status.StatusStr;
	PropertiesV1toV2(Presence.Status.Properties, PresenceV2->Properties);

	return PresenceV2;
}

TSharedRef<FOnlineUserPresence> FPresenceOSSAdapter::PresenceV2toV1(FUserPresence& Presence)
{
	TSharedRef<FOnlineUserPresence> PresenceV1 = MakeShared<FOnlineUserPresence>();

	PresenceV1->bIsOnline = (Presence.Status == EUserPresenceStatus::Online) ? 1 : 0;
	PresenceV1->bIsPlaying = (Presence.GameStatus != EUserPresenceGameStatus::Unknown) ? 1 : 0;
	PresenceV1->bIsPlayingThisGame = (Presence.GameStatus == EUserPresenceGameStatus::PlayingThisGame) ? 1 : 0;
	PresenceV1->bIsJoinable = (Presence.Joinability == EUserPresenceJoinability::Public) ? 1 : 0;
	PresenceV1->Status.StatusStr = Presence.StatusString;
	PresenceV1->Status.State = StatusV2toV1(Presence.Status);
	PropertiesV2toV1(Presence.Properties, PresenceV1->Status.Properties);

	return PresenceV1;
}

void FPresenceOSSAdapter::PostInitialize()
{
	Auth = Services.Get<FAuthOSSAdapter>();
	PresenceInt = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem().GetPresenceInterface();
}

void FPresenceOSSAdapter::PreShutdown()
{
}

TOnlineAsyncOpHandle<FQueryPresence> FPresenceOSSAdapter::QueryPresence(FQueryPresence::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryPresence> Op = GetOp<FQueryPresence>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FQueryPresence>& Op)
	{
		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		const FUniqueNetIdPtr UniqueNetId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (UniqueNetId)
		{
			PresenceInt->QueryPresence(*UniqueNetId, *MakeDelegateAdapter(this, [this, WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise)](const FUniqueNetId& UserId, const bool bWasSuccessful) mutable
			{
				if (TOnlineAsyncOpPtr<FQueryPresence> StrongOp = WeakOp.Pin())
				{
					if (bWasSuccessful)
					{
						FQueryPresence::Result Result;
						StrongOp->SetResult(MoveTemp(Result));
					}
					else
					{
						StrongOp->SetError(Errors::Unknown());
					}
				}

				Promise.EmplaceValue();
			}));
		}
		else
		{
			Promise.EmplaceValue();
		}

		return Future;
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

// Base implementation for the batch query function makes multiple async calls to query and returns them as a single output
TOnlineAsyncOpHandle<FBatchQueryPresence> FPresenceOSSAdapter::BatchQueryPresence(FBatchQueryPresence::Params&& Params)
{
	TOnlineAsyncOpRef<FBatchQueryPresence> Op = GetOp<FBatchQueryPresence>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FBatchQueryPresence>& Op)
	{
		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		const FUniqueNetIdPtr LocalUserId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			Promise.EmplaceValue();
			return Future;
		}

		TArray<FUniqueNetIdRef> NetIds;
		for (const FAccountId& TargetAccountId : Op.GetParams().TargetAccountIds)
		{
			const FUniqueNetIdPtr TargetUserNetId = Auth->GetUniqueNetId(TargetAccountId);
			if (!TargetUserNetId)
			{
				Op.SetError(Errors::InvalidParams());
				Promise.EmplaceValue();
				return Future;
			}

			NetIds.Add(TargetUserNetId.ToSharedRef());
		}

		PresenceInt->QueryPresence(*LocalUserId, NetIds, *MakeDelegateAdapter(this, [this, WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise)](const FUniqueNetId& UserId, const bool bWasSuccessful) mutable
		{
			if (TOnlineAsyncOpPtr<FBatchQueryPresence> StrongOp = WeakOp.Pin())
			{
				if (bWasSuccessful)
				{
					FBatchQueryPresence::Result Result;
					StrongOp->SetResult(MoveTemp(Result));
				}
				else
				{
					StrongOp->SetError(Errors::Unknown());
				}
			}

			Promise.EmplaceValue();
		}));

		return Future;
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineResult<FGetCachedPresence> FPresenceOSSAdapter::GetCachedPresence(FGetCachedPresence::Params&& Params)
{
	const FUniqueNetId& User = *Auth->GetUniqueNetId(Params.LocalAccountId);
	TSharedPtr<FOnlineUserPresence> Presence = MakeShared<FOnlineUserPresence>();
	EOnlineCachedResult::Type Result = PresenceInt->GetCachedPresence(User, Presence);
	if (Result == EOnlineCachedResult::Success)
	{
		TSharedRef<FUserPresence> V2Presence = PresenceV1toV2(Params.TargetAccountId, *Presence);
		return TOnlineResult<FGetCachedPresence>({V2Presence});
	}
	else
	{
		return TOnlineResult<FGetCachedPresence>(Errors::NotFound());
	}
}

TOnlineAsyncOpHandle<FUpdatePresence> FPresenceOSSAdapter::UpdatePresence(FUpdatePresence::Params&& Params)
{
	TOnlineAsyncOpRef<FUpdatePresence> Op = GetOp<FUpdatePresence>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FUpdatePresence>& Op)
	{
		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		TSharedRef<const FOnlineUserPresence> PresenceV1 = PresenceV2toV1(*Op.GetParams().Presence);
		PresenceInt->SetPresence(*Auth->GetUniqueNetId(Op.GetParams().LocalAccountId), PresenceV1->Status, *MakeDelegateAdapter(this, [this, WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise)](const FUniqueNetId& UserId, const bool bWasSuccessful) mutable
		{
			if (TOnlineAsyncOpPtr<FUpdatePresence> StrongOp = WeakOp.Pin())
			{
				if (bWasSuccessful)
				{
					FUpdatePresence::Result Result;
					StrongOp->SetResult(MoveTemp(Result));
				}
				else
				{
					StrongOp->SetError(Errors::Unknown());
				}
			}

			Promise.EmplaceValue();
		}));

		return Future;
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FPartialUpdatePresence> FPresenceOSSAdapter::PartialUpdatePresence(FPartialUpdatePresence::Params&& Params)
{
	TOnlineAsyncOpRef<FPartialUpdatePresence> Op = GetOp<FPartialUpdatePresence>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FPartialUpdatePresence>& Op)
	{
		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		const FPartialUpdatePresence::Params::FMutations& Mutations = Op.GetParams().Mutations;
		FOnlinePresenceSetPresenceParameters Parameters;
	
		if (Mutations.StatusString.IsSet())
		{
			Parameters.StatusStr = Mutations.StatusString.GetValue();
		}

		if (Mutations.Status.IsSet())
		{
			Parameters.State = StatusV2toV1(Mutations.Status.GetValue());
		}

		// todo: V1 doesnt have a clear path for doing key removals
		if (Mutations.UpdatedProperties.Num() > 0)
		{
			FOnlineKeyValuePairs<FPresenceKey, FVariantData> NewProperties;
			PropertiesV2toV1(Mutations.UpdatedProperties, NewProperties);
			Parameters.Properties = NewProperties;
		}

		PresenceInt->SetPresence(*Auth->GetUniqueNetId(Op.GetParams().LocalAccountId), MoveTemp(Parameters), *MakeDelegateAdapter(this, [this, WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise)](const FUniqueNetId& UserId, const bool bWasSuccessful) mutable
		{
			if (TOnlineAsyncOpPtr<FPartialUpdatePresence> StrongOp = WeakOp.Pin())
			{
				if (bWasSuccessful)
				{
					FPartialUpdatePresence::Result Result;
					StrongOp->SetResult(MoveTemp(Result));
				}
				else
				{
					StrongOp->SetError(Errors::Unknown());
				}
			}

			Promise.EmplaceValue();
		}));

		return Future;
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

/* UE::Online */ }
