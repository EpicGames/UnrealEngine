// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/PrivilegesOSSAdapter.h"

#include "Online/OnlineServicesOSSAdapter.h"
#include "Online/AuthOSSAdapter.h"

#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"

namespace UE::Online {

void FPrivilegesOSSAdapter::PostInitialize()
{
	Super::PostInitialize();
}

void FPrivilegesOSSAdapter::PreShutdown()
{
	Super::PreShutdown();
}

TOnlineAsyncOpHandle<FQueryUserPrivilege> FPrivilegesOSSAdapter::QueryUserPrivilege(FQueryUserPrivilege::Params&& Params)
{
	TSharedRef<TOnlineAsyncOp<FQueryUserPrivilege>> Op = GetOp<FQueryUserPrivilege>(MoveTemp(Params));

	if (!Op->IsReady())
	{
		FAuthOSSAdapter* Auth = Services.Get<FAuthOSSAdapter>();
		FUniqueNetIdPtr UniqueNetId = Auth->GetUniqueNetId(Op->GetParams().LocalAccountId);
		if (!UniqueNetId.IsValid())
		{
			Op->SetError(Errors::InvalidUser());
			return Op->GetHandle();
		}

		Op->Then([this, UniqueNetId](TOnlineAsyncOp<FQueryUserPrivilege>& Op)
			{
				TSharedPtr<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				::EUserPrivileges::Type Privilege;
				switch (Op.GetParams().Privilege)
				{
					default:
					case EUserPrivileges::CanPlay:						Privilege = ::EUserPrivileges::CanPlay;						break;
					case EUserPrivileges::CanPlayOnline:				Privilege = ::EUserPrivileges::CanPlayOnline;				break;
					case EUserPrivileges::CanCommunicateViaTextOnline:	Privilege = ::EUserPrivileges::CanCommunicateOnline;		break;
					case EUserPrivileges::CanCommunicateViaVoiceOnline:	Privilege = ::EUserPrivileges::CanCommunicateOnline;		break;
					case EUserPrivileges::CanUseUserGeneratedContent:	Privilege = ::EUserPrivileges::CanUseUserGeneratedContent;	break;
					case EUserPrivileges::CanCrossPlay:					Privilege = ::EUserPrivileges::CanUserCrossPlay;			break;
				}

				GetIdentityInterface()->GetUserPrivilege(*UniqueNetId, Privilege, 
					IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda(
						[WeakOp = Op.AsWeak(), Promise](const FUniqueNetId& LocalUserId, ::EUserPrivileges::Type Privilege, uint32 PrivilegeResult)
						{
							TSharedPtr<TOnlineAsyncOp<FQueryUserPrivilege>> PinnedOp = WeakOp.Pin();
							if (PinnedOp.IsValid())
							{
								FQueryUserPrivilege::Result Result;
								// OSS EPrivilegeResults currently has the same values as OnlineServices EPrivilegeResults
								Result.PrivilegeResult = static_cast<EPrivilegeResults>(PrivilegeResult);
								PinnedOp->SetResult(MoveTemp(Result));

								Promise->SetValue();
							}
						}), EShowPrivilegeResolveUI::Default);

				return Promise->GetFuture();
			})
			.Enqueue(GetSerialQueue());
	}

	return Op->GetHandle();
}

IOnlineIdentityPtr FPrivilegesOSSAdapter::GetIdentityInterface() const
{
	return static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem().GetIdentityInterface();
}

/* UE::Online */ }
