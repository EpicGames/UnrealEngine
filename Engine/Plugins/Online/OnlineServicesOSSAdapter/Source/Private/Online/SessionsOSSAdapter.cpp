// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SessionsOSSAdapter.h"

#include "OnlineError.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Interfaces/OnlineIdentityInterface.h"

#include "Online/OnlineServicesOSSAdapter.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/UserInfoOSSAdapter.h"
#include "Online/DelegateAdapter.h"
#include "Online/ErrorsOSSAdapter.h"
#include "Online/OnlineSessionNames.h"

#include "Algo/Find.h"

namespace UE::Online {

/** Auxiliary functions */

EOnlineComparisonOp::Type GetV1SessionSearchComparisonOp(const ESchemaAttributeComparisonOp& InValue)
{
	switch (InValue)
	{
	case ESchemaAttributeComparisonOp::Equals:			return EOnlineComparisonOp::Equals;
	case ESchemaAttributeComparisonOp::GreaterThan:		return EOnlineComparisonOp::GreaterThan;
	case ESchemaAttributeComparisonOp::GreaterThanEquals:	return EOnlineComparisonOp::GreaterThanEquals;
	case ESchemaAttributeComparisonOp::LessThan:			return EOnlineComparisonOp::LessThan;
	case ESchemaAttributeComparisonOp::LessThanEquals:	return EOnlineComparisonOp::LessThanEquals;
	case ESchemaAttributeComparisonOp::Near:				return EOnlineComparisonOp::Near;
	case ESchemaAttributeComparisonOp::NotEquals:			return EOnlineComparisonOp::NotEquals;
	}

	checkNoEntry();

	return EOnlineComparisonOp::Equals;
}

FSchemaVariant GetV2SessionVariant(const FVariantData& InValue)
{
	FSchemaVariant Result;
	
	switch(InValue.GetType())
	{
	case EOnlineKeyValuePairDataType::Int32:
	{
		int32 Value;
		InValue.GetValue(Value);
		Result.Set((int64)Value);
		break;
	}
	case EOnlineKeyValuePairDataType::UInt32:
	{
		uint32 Value;
		InValue.GetValue(Value);
		Result.Set((int64)Value);
		break;
	}
	case EOnlineKeyValuePairDataType::Int64:
	{
		int64 Value;
		InValue.GetValue(Value);
		Result.Set(Value);
		break;
	}
	case EOnlineKeyValuePairDataType::UInt64:
	{
		uint64 Value;
		InValue.GetValue(Value);
		Result.Set((int64)Value);
		break;
	}
	case EOnlineKeyValuePairDataType::Float:
	{
		float Value;
		InValue.GetValue(Value);
		Result.Set((double)Value);
		break;
	}	
	case EOnlineKeyValuePairDataType::Double:
	{
		double Value;
		InValue.GetValue(Value);
		Result.Set(Value);
		break;
	}
	case EOnlineKeyValuePairDataType::String:
	{
		FString Value;
		InValue.GetValue(Value);
		Result.Set(Value);
		break;
	}
	case EOnlineKeyValuePairDataType::Bool:
	{
		bool Value;
		InValue.GetValue(Value);
		Result.Set(Value);
		break;
	}
	case EOnlineKeyValuePairDataType::Empty: // Intentional fallthrough
	case EOnlineKeyValuePairDataType::Blob:	// Intentional fallthrough
	case EOnlineKeyValuePairDataType::Json:	// Intentional fallthrough
	case EOnlineKeyValuePairDataType::MAX:	// Intentional fallthrough
	default:
		UE_LOGF(LogOnlineServices, Warning, "[GetV2SessionVariant] FVariantData type not supported by FSessionsVariant. No value was set.");
		break;
	}

	return Result;
}

FVariantData GetV1VariantData(const FSchemaVariant& InValue)
{
	FVariantData Result;

	switch (InValue.VariantType)
	{
	case ESchemaAttributeType::Bool:
		Result.SetValue(InValue.GetBoolean());
		break;
	case ESchemaAttributeType::Double:
		Result.SetValue(InValue.GetDouble());
		break;
	case ESchemaAttributeType::Int64:
		Result.SetValue(InValue.GetInt64());
		break;
	case ESchemaAttributeType::String:
		Result.SetValue(InValue.GetString());
		break;
	default:
		UE_LOGF(LogOnlineServices, Warning, "[GetV1VariantData] FSchemaVariant type not supported by FVariantData. No value was set.");
		break;
	}

	return Result;
}

ESchemaAttributeVisibility GetV2SessionsAttributeVisibility(const EOnlineDataAdvertisementType::Type& InValue)
{
	switch (InValue)
	{
	case EOnlineDataAdvertisementType::DontAdvertise:			// Intentional fallthrough
	case EOnlineDataAdvertisementType::ViaPingOnly:				return ESchemaAttributeVisibility::Private;
	case EOnlineDataAdvertisementType::ViaOnlineService:		// Intentional fallthrough
	case EOnlineDataAdvertisementType::ViaOnlineServiceAndPing:	return ESchemaAttributeVisibility::Public;
	}

	checkNoEntry();
	return ESchemaAttributeVisibility::Private;
}

EOnlineDataAdvertisementType::Type GetV1OnlineDataAdvertisementType(const ESchemaAttributeVisibility& InValue)
{
	switch (InValue)
	{
	case ESchemaAttributeVisibility::Private:	return EOnlineDataAdvertisementType::DontAdvertise;
	case ESchemaAttributeVisibility::Public:	return EOnlineDataAdvertisementType::ViaOnlineServiceAndPing;
	}

	checkNoEntry();
	return EOnlineDataAdvertisementType::DontAdvertise;
}

FCustomSessionSettingsMap GetV2SessionSettings(const ::FSessionSettings& InSessionSettings)
{
	FCustomSessionSettingsMap Result;

	for (const TPair<FName, FOnlineSessionSetting>& Entry : InSessionSettings)
	{
		FCustomSessionSetting NewCustomSetting;
		NewCustomSetting.Data = GetV2SessionVariant(Entry.Value.Data);
		NewCustomSetting.Visibility = GetV2SessionsAttributeVisibility(Entry.Value.AdvertisementType);
		NewCustomSetting.ID = Entry.Value.ID;

		Result.Emplace(Entry.Key, NewCustomSetting);
	}

	return Result;
}

::FSessionSettings GetV1SessionSettings(const FCustomSessionSettingsMap& InSessionSettings)
{
	::FSessionSettings Result;

	for (const TPair<FSchemaAttributeId, FCustomSessionSetting>& Entry : InSessionSettings)
	{
		FOnlineSessionSetting NewSessionSetting;
		NewSessionSetting.Data = GetV1VariantData(Entry.Value.Data);
		NewSessionSetting.AdvertisementType = GetV1OnlineDataAdvertisementType(Entry.Value.Visibility);
		NewSessionSetting.ID = Entry.Value.ID;

		Result.Add(Entry.Key, NewSessionSetting);
	}

	return Result;
}

TSharedRef<FOnlineSessionSearch> BuildV1SessionSearch(const FFindSessions::Params& SearchParams)
{
	TSharedRef<FOnlineSessionSearch> Result = MakeShared<FOnlineSessionSearch>();

	Result->bIsLanQuery = SearchParams.bFindLANSessions;

	Result->MaxSearchResults = SearchParams.MaxResults;

	if (const FFindSessionsSearchFilter* PingBucketSize = Algo::FindBy(SearchParams.Filters, OSS_ADAPTER_SESSION_SEARCH_PING_BUCKET_SIZE, &FFindSessionsSearchFilter::Key))
	{
		Result->PingBucketSize = (int32)PingBucketSize->Value.GetInt64();
	}

	if (const FFindSessionsSearchFilter* PlatformHash = Algo::FindBy(SearchParams.Filters, OSS_ADAPTER_SESSION_SEARCH_PLATFORM_HASH, &FFindSessionsSearchFilter::Key))
	{
		Result->PlatformHash = (int32)PlatformHash->Value.GetInt64();
	}

	for (const FFindSessionsSearchFilter& SearchFilter : SearchParams.Filters)
	{
		switch (SearchFilter.Value.VariantType)
		{
		case ESchemaAttributeType::Bool:
			Result->QuerySettings.Set<bool>(SearchFilter.Key, SearchFilter.Value.GetBoolean(), GetV1SessionSearchComparisonOp(SearchFilter.ComparisonOp));
			break;
		case ESchemaAttributeType::Double:
			Result->QuerySettings.Set<double>(SearchFilter.Key, SearchFilter.Value.GetDouble(), GetV1SessionSearchComparisonOp(SearchFilter.ComparisonOp));
			break;
		case ESchemaAttributeType::Int64:
			Result->QuerySettings.Set<uint64>(SearchFilter.Key, (uint64)SearchFilter.Value.GetInt64(), GetV1SessionSearchComparisonOp(SearchFilter.ComparisonOp));
			break;
		case ESchemaAttributeType::String:
			Result->QuerySettings.Set<FString>(SearchFilter.Key, SearchFilter.Value.GetString(), GetV1SessionSearchComparisonOp(SearchFilter.ComparisonOp));
			break;
		}
	}

	return Result;
}

/** FSessionOSSAdapter */

void WriteV2SessionInfoFromV1Session(const FOnlineSession& InSession, FSessionInfo& OutInfo)
{
	OutInfo.SessionIdOverride = InSession.SessionSettings.SessionIdOverride;
	bool bAllowSanctionedPlayers = false;
	InSession.SessionSettings.Get(OSS_ADAPTER_SESSIONS_ALLOW_SANCTIONED_PLAYERS, bAllowSanctionedPlayers);
	OutInfo.bAllowSanctionedPlayers = bAllowSanctionedPlayers;

	OutInfo.bAntiCheatProtected = InSession.SessionSettings.bAntiCheatProtected;

	OutInfo.bIsDedicatedServerSession = InSession.SessionSettings.bIsDedicated;

	OutInfo.bIsLANSession = InSession.SessionSettings.bIsLANMatch;
}

FSessionOSSAdapter::FSessionOSSAdapter(const FOnlineSession& InSession, const FOnlineSessionId& InSessionId)
	: V1Session(InSession)
{
	WriteV2SessionInfoFromV1Session(InSession, SessionInfo);
	SessionInfo.SessionId = InSessionId;
}

const FSessionOSSAdapter& FSessionOSSAdapter::Cast(const ISession& InSession)
{
	// TODO: Check for type matching with SessionId

	return static_cast<const FSessionOSSAdapter&>(InSession);
}

const FOnlineSession& FSessionOSSAdapter::GetV1Session() const
{
	return V1Session;
}

void FSessionOSSAdapter::DumpState() const
{
	FSessionCommon::DumpState();
	UE_LOGF(LogOnlineServices, Log, "V1Session: SessionId [%ls]", *V1Session.GetSessionIdStr());
}

/** FSessionsOSSAdapter */ 

FSessionsOSSAdapter::FSessionsOSSAdapter(FOnlineServicesOSSAdapter& InServices)
	: Super(InServices)
{
}

void FSessionsOSSAdapter::Initialize()
{
	Super::Initialize();

	IOnlineSubsystem& SubsystemV1 = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem();

	SessionsInterface = SubsystemV1.GetSessionInterface();
	check(SessionsInterface);

	IdentityInterface = SubsystemV1.GetIdentityInterface();
	check(IdentityInterface);

	// OnSessionInviteReceived is not used by current implementations of Sessions in V1, so we won't bind to FOnSessionInviteReceived

	SessionsInterface->AddOnSessionUserInviteAcceptedDelegate_Handle(FOnSessionUserInviteAcceptedDelegate::CreateLambda([this](const bool bWasSuccessful, const int32 ControllerId, FUniqueNetIdPtr InvitedUserIdPtr, const FOnlineSessionSearchResult& InviteResult)
	{
		FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

		FUniqueNetIdRef LocalAccountId = FUniqueNetIdString::EmptyId();
		FUniqueNetIdPtr LocalAccountIdPtr = IdentityInterface->GetUniquePlayerId(ControllerId);
		if (LocalAccountIdPtr.IsValid())
		{
			LocalAccountId = (*LocalAccountIdPtr).AsShared();
		}

		TSharedRef<FSessionCommon> V2Session = BuildV2Session(&InviteResult.Session);
		const FAccountId LocalAccountIdV2 = ServicesOSSAdapter.GetAccountIdRegistry().FindOrAddHandle(LocalAccountId);
		AddSearchResult(V2Session, LocalAccountIdV2);

		FUISessionJoinRequested Event {
			LocalAccountIdV2,
			TResult<FOnlineSessionId, FOnlineError>(GetSessionIdRegistry().FindOrAddHandle(InviteResult.Session.SessionInfo->GetSessionId().AsShared())),
			EUISessionJoinRequestedSource::FromInvitation
		};

		SessionEvents.OnUISessionJoinRequested.Broadcast(Event);
	}));

	SessionsInterface->AddOnCreateSessionCompleteDelegate_Handle(FOnCreateSessionCompleteDelegate::CreateLambda([this](FName SessionName, bool bSessionCompletionResult)
	{
		if (bSessionCompletionResult)
		{
			FAccountId AccountId;
			FOnlineSessionId OnlineSessionId;
			if (GetAccountIdAndSessionIdFromNamedSession(SessionName, AccountId, OnlineSessionId))
			{
				const FSessionCreated Event { AccountId, OnlineSessionId };
				SessionEvents.OnSessionCreated.Broadcast(Event);
			}
		}
	}));

	SessionsInterface->AddOnJoinSessionCompleteDelegate_Handle(FOnJoinSessionCompleteDelegate::CreateLambda([this](FName SessionName, EOnJoinSessionCompleteResult::Type JoinSessionType)
	{
		if (JoinSessionType == EOnJoinSessionCompleteResult::Type::Success)
		{
			FAccountId AccountId;
			FOnlineSessionId OnlineSessionId;
			if (GetAccountIdAndSessionIdFromNamedSession(SessionName, AccountId, OnlineSessionId))
			{
				const FSessionJoined Event { AccountId, OnlineSessionId };
				SessionEvents.OnSessionJoined.Broadcast(Event);
			}
		}
	}));

	SessionsInterface->AddOnSessionParticipantJoinedDelegate_Handle(FOnSessionParticipantJoinedDelegate::CreateLambda([this](FName SessionName, const FUniqueNetId& TargetUniqueNetId)
	{
		// We won't update a session that can't be retrieved from the OSS or doesn't exist in OnlineServices anymore
		if (UpdateV2Session(SessionName))
		{
			const FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);
			const FUniqueNetIdRef TargetUniqueNetIdRef = TargetUniqueNetId.AsShared();
			const FAccountId TargetAccountId = ServicesOSSAdapter.GetAccountIdRegistry().FindOrAddHandle(TargetUniqueNetIdRef);

			FSessionUpdate SessionUpdate;
			SessionUpdate.AddedSessionMembers.Emplace(TargetAccountId);

			const FSessionUpdated Event{ SessionName , SessionUpdate };

			SessionEvents.OnSessionUpdated.Broadcast(Event);
		}
	}));

	SessionsInterface->AddOnSessionParticipantLeftDelegate_Handle(FOnSessionParticipantLeftDelegate::CreateLambda([this](FName SessionName, const FUniqueNetId& TargetUniqueNetId, EOnSessionParticipantLeftReason LeaveReason)
	{
		// We won't update a session that can't be retrieved from the OSS or doesn't exist in OnlineServices anymore
		if (UpdateV2Session(SessionName))
		{
			const FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);
			const FUniqueNetIdRef TargetUniqueNetIdRef = TargetUniqueNetId.AsShared();
			const FAccountId TargetAccountId = ServicesOSSAdapter.GetAccountIdRegistry().FindOrAddHandle(TargetUniqueNetIdRef);

			FSessionUpdate SessionUpdate;
			SessionUpdate.RemovedSessionMembers.Emplace(TargetAccountId);

			const FSessionUpdated Event{ SessionName , SessionUpdate };

			SessionEvents.OnSessionUpdated.Broadcast(Event);
		}
	}));

	// Session member settings don't exist in V2, so we won't bind to FOnSessionParticipantSettingsUpdated
}

void FSessionsOSSAdapter::Shutdown()
{
	SessionsInterface->ClearOnSessionInviteReceivedDelegates(this);
	SessionsInterface->ClearOnSessionUserInviteAcceptedDelegates(this);
	SessionsInterface->ClearOnSessionParticipantJoinedDelegates(this);
	SessionsInterface->ClearOnSessionParticipantLeftDelegates(this);
	SessionsInterface->ClearOnSessionParticipantSettingsUpdatedDelegates(this);
	SessionsInterface->ClearOnCreateSessionCompleteDelegates(this);
	SessionsInterface->ClearOnJoinSessionCompleteDelegates(this);

	Super::Shutdown();
}

TFuture<TOnlineResult<FCreateSession>> FSessionsOSSAdapter::CreateSessionImpl(const FCreateSession::Params& Params)
{

	TPromise<TOnlineResult<FCreateSession>> Promise;
	TFuture<TOnlineResult<FCreateSession>> Future = Promise.GetFuture();

	// Dedicated servers can update sessions so we won't check if LocalAccountId is valid
	const FUniqueNetIdRef LocalAccountId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId).ToSharedRef();

	MakeMulticastAdapter(this, SessionsInterface->OnCreateSessionCompleteDelegates,
	[this, Promise = MoveTemp(Promise), Params](FName SessionName, const bool bWasSuccessful) mutable
	{
		if (!bWasSuccessful)
		{
			Promise.EmplaceValue(Errors::RequestFailure());
			return;
		}

		const FNamedOnlineSession* V1Session = SessionsInterface->GetNamedSession(SessionName);

		TSharedRef<FSessionCommon> V2Session = BuildV2Session(V1Session);

		AddSessionWithReferences(V2Session, Params.SessionName, Params.LocalAccountId, Params.bPresenceEnabled);

		Promise.EmplaceValue(FCreateSession::Result{ });
	});

	SessionsInterface->CreateSession(*LocalAccountId, Params.SessionName, BuildV1SettingsForCreate(Params));

	return Future;
}

TFuture<TOnlineResult<FUpdateSessionSettings>> FSessionsOSSAdapter::UpdateSessionSettingsImpl(const FUpdateSessionSettings::Params& Params)
{
	TPromise<TOnlineResult<FUpdateSessionSettings>> Promise;
	TFuture<TOnlineResult<FUpdateSessionSettings>> Future = Promise.GetFuture();

	MakeMulticastAdapter(this, SessionsInterface->OnUpdateSessionCompleteDelegates,
	[this, Promise = MoveTemp(Promise), OpParams = Params](FName SessionName, const bool bWasSuccessful) mutable
	{
		if (!bWasSuccessful)
		{
			Promise.EmplaceValue(Errors::RequestFailure());
			return;
		}

		TOnlineResult<FGetMutableSessionByName> GetMutableSessionByNameResult = GetMutableSessionByName({ SessionName });
		if (GetMutableSessionByNameResult.IsOk())
		{
			// We update our local session
			TSharedRef<FSessionCommon>& FoundSession = GetMutableSessionByNameResult.GetOkValue().Session;

			FSessionUpdate SessionUpdateData = BuildSessionUpdate(FoundSession, OpParams.Mutations);

			(*FoundSession) += SessionUpdateData;

			// We set the result and fire the event
			Promise.EmplaceValue(FUpdateSessionSettings::Result{ });

			FSessionUpdated SessionUpdatedEvent{ OpParams.SessionName, SessionUpdateData };
			SessionEvents.OnSessionUpdated.Broadcast(SessionUpdatedEvent);
		}
		else
		{
			Promise.EmplaceValue(Errors::NotFound());
		}
	});

	const TOnlineResult<FGetMutableSessionByName> GetMutableSessionByNameResult = GetMutableSessionByName({ Params.SessionName });
	check(GetMutableSessionByNameResult.IsOk());
		
	const TSharedRef<FSessionCommon>& FoundSession = GetMutableSessionByNameResult.GetOkValue().Session;

	// We will update a copy here, and wait until the operation has completed successfully to update our local data
	FSessionSettings UpdatedV2Settings = FoundSession->SessionSettings;
	FSessionUpdate SessionUpdate = BuildSessionUpdate(FoundSession, Params.Mutations);
	check(SessionUpdate.SessionSettingsChanges.IsSet());
	UpdatedV2Settings += SessionUpdate.SessionSettingsChanges.GetValue();

	FOnlineSessionSettings UpdatedV1Settings = BuildV1SettingsForUpdate(Params.LocalAccountId, FoundSession, UpdatedV2Settings);

	SessionsInterface->UpdateSession(Params.SessionName, UpdatedV1Settings);

	return Future;
}

TFuture<TOnlineResult<FLeaveSession>> FSessionsOSSAdapter::LeaveSessionImpl(const FLeaveSession::Params& Params)
{
	TPromise<TOnlineResult<FLeaveSession>> Promise;
	TFuture<TOnlineResult<FLeaveSession>> Future = Promise.GetFuture();

	MakeMulticastAdapter(this, SessionsInterface->OnDestroySessionCompleteDelegates,
	[this, Promise = MoveTemp(Promise), Params](FName SessionName, const bool bWasSuccessful) mutable
	{
		if (!bWasSuccessful)
		{
			Promise.EmplaceValue(Errors::RequestFailure());
			SessionsInterface->ClearOnDestroySessionCompleteDelegates(this);
			return;
		}

		TOnlineResult<FGetSessionByName> GetSessionByNameResult = GetSessionByName({ Params.SessionName });
		if (GetSessionByNameResult.IsOk())
		{
			TSharedRef<const ISession> FoundSession = GetSessionByNameResult.GetOkValue().Session;

			ClearSessionReferences(FoundSession->GetSessionId(), Params.SessionName, Params.LocalAccountId);

			Promise.EmplaceValue(FLeaveSession::Result{ });

			FSessionLeft SessionLeftEvent;
			SessionLeftEvent.LocalAccountId = Params.LocalAccountId;

			SessionEvents.OnSessionLeft.Broadcast(SessionLeftEvent);
		}
		else
		{
			Promise.EmplaceValue(GetSessionByNameResult.GetErrorValue());
		}
	});

	SessionsInterface->DestroySession(Params.SessionName);

	return Future;
}

TFuture<TOnlineResult<FFindSessions>> FSessionsOSSAdapter::FindSessionsImpl(const FFindSessions::Params& Params)
{
	TPromise<TOnlineResult<FFindSessions>> Promise;
	TFuture<TOnlineResult<FFindSessions>> Future = Promise.GetFuture();

	// Before we start the search, we reset the cache and save the promise
	SearchResultsUserMap.FindOrAdd(Params.LocalAccountId).Reset();
	CurrentSessionSearchPromisesUserMap.Emplace(Params.LocalAccountId, MoveTemp(Promise));

	const FUniqueNetIdPtr LocalAccountIdPtr = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
	check(LocalAccountIdPtr.IsValid());
	const FUniqueNetIdRef LocalAccountId = LocalAccountIdPtr.ToSharedRef();

	if (Params.SessionId.IsSet())
	{
		if (const FUniqueNetIdPtr SessionIdPtr = GetSessionIdRegistry().GetIdValue(*Params.SessionId))
		{
			const FUniqueNetIdRef SessionIdRef = SessionIdPtr.ToSharedRef();

			MakeMulticastAdapter(this, SessionsInterface->OnFindSessionsCompleteDelegates,
			[this, Params](bool bWasSuccessful)
			{
				TPromise<TOnlineResult<FFindSessions>>& Promise = CurrentSessionSearchPromisesUserMap.FindChecked(Params.LocalAccountId);

				if (bWasSuccessful)
				{
					TSharedRef<FOnlineSessionSearch>& PendingV1SessionSearch = PendingV1SessionSearchesPerUser.FindChecked(Params.LocalAccountId);

					TArray<TSharedRef<FSessionCommon>> FoundSessions = BuildV2SessionSearchResults(PendingV1SessionSearch->SearchResults);
					for (const TSharedRef<FSessionCommon>& FoundSession : FoundSessions)
					{
						AddSearchResult(FoundSession, Params.LocalAccountId);
					}

					TArray<FOnlineSessionId> SearchResults = SearchResultsUserMap.FindOrAdd(Params.LocalAccountId);
					Promise.EmplaceValue(FFindSessions::Result{ SearchResults });
				}
				else
				{
					Promise.EmplaceValue(Errors::RequestFailure());
				}

				CurrentSessionSearchPromisesUserMap.Remove(Params.LocalAccountId);
				PendingV1SessionSearchesPerUser.Remove(Params.LocalAccountId);
			});

			FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

			FUniqueNetIdRef TargetIdRef = FUniqueNetIdString::EmptyId();
			if (Params.TargetUser.IsSet())
			{
				TargetIdRef = ServicesOSSAdapter.GetAccountIdRegistry().GetIdValue(*Params.TargetUser).ToSharedRef();
			}

			PendingV1SessionSearchesPerUser.Emplace(Params.LocalAccountId, MakeShared<FOnlineSessionSearch>());

			SessionsInterface->FindSessionById(*LocalAccountId, *SessionIdRef, *TargetIdRef, *MakeDelegateAdapter(this, [this, Params](int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& SingleSearchResult) mutable
			{
				TPromise<TOnlineResult<FFindSessions>>& Promise = CurrentSessionSearchPromisesUserMap.FindChecked(Params.LocalAccountId);

				if (!bWasSuccessful)
				{
					Promise.EmplaceValue(Errors::RequestFailure());
					return;
				}

				TSharedRef<FOnlineSessionSearch>& PendingV1SessionSearch = PendingV1SessionSearchesPerUser.FindChecked(Params.LocalAccountId);

				PendingV1SessionSearch->SearchResults.Add(SingleSearchResult);
			}));
		}
		else
		{
			UE_LOGF(LogOnlineServices, Verbose, "[FSessionsOSSAdapter::FindSessionsImpl] UniqueNetId could not be found for SessionId %ls", *ToLogString(*Params.SessionId));
			Promise.EmplaceValue(Errors::InvalidParams());
			return Future;
		}
	}
	else if (Params.TargetUser.IsSet()) 	// If there is a target user, we will use FindFriendSession
	{
		int32 LocalUserNum = Services.Get<FAuthOSSAdapter>()->GetLocalUserNum(Params.LocalAccountId);

		MakeMulticastAdapter(this, SessionsInterface->OnFindFriendSessionCompleteDelegates[LocalUserNum],
		[this, Params](int32 LocalAccountId, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& FriendSearchResults)
		{
			TPromise<TOnlineResult<FFindSessions>>& Promise = CurrentSessionSearchPromisesUserMap.FindChecked(Params.LocalAccountId);

			if (bWasSuccessful)
			{
				TArray<TSharedRef<FSessionCommon>> FoundSessions = BuildV2SessionSearchResults(FriendSearchResults);
				for (const TSharedRef<FSessionCommon>& FoundSession : FoundSessions)
				{
					AddSearchResult(FoundSession,Params.LocalAccountId);
				}

				TArray<FOnlineSessionId> SearchResults = SearchResultsUserMap.FindOrAdd(Params.LocalAccountId);
				Promise.EmplaceValue(FFindSessions::Result{ SearchResults });
			}
			else
			{
				Promise.EmplaceValue(Errors::RequestFailure());
			}

			CurrentSessionSearchPromisesUserMap.Remove(Params.LocalAccountId);
			PendingV1SessionSearchesPerUser.Remove(Params.LocalAccountId);
		});

		FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

		TArray<FUniqueNetIdRef> TargetUniqueNetIds = { ServicesOSSAdapter.GetAccountIdRegistry().GetIdValue(*Params.TargetUser).ToSharedRef() };

		// The cached search information is not used in this mode, but we'll still update it to not have more than one search at the same time
		PendingV1SessionSearchesPerUser.Emplace(Params.LocalAccountId, MakeShared<FOnlineSessionSearch>());

		SessionsInterface->FindFriendSession(*LocalAccountId, TargetUniqueNetIds);
	}
	else
	{
		MakeMulticastAdapter(this, SessionsInterface->OnFindSessionsCompleteDelegates,
		[this, Params](bool bWasSuccessful) mutable
		{
			TPromise<TOnlineResult<FFindSessions>>& Promise = CurrentSessionSearchPromisesUserMap.FindChecked(Params.LocalAccountId);

			if (bWasSuccessful)
			{
				TSharedRef<FOnlineSessionSearch>& PendingV1SessionSearch = PendingV1SessionSearchesPerUser.FindChecked(Params.LocalAccountId);

				TArray<TSharedRef<FSessionCommon>> FoundSessions = BuildV2SessionSearchResults(PendingV1SessionSearch->SearchResults);
				for (const TSharedRef<FSessionCommon>& FoundSession : FoundSessions)
				{
					AddSearchResult(FoundSession, Params.LocalAccountId);
				}

				TArray<FOnlineSessionId> SearchResults = SearchResultsUserMap.FindOrAdd(Params.LocalAccountId);
				Promise.EmplaceValue(FFindSessions::Result{ SearchResults });
			}
			else
			{
				Promise.EmplaceValue(Errors::RequestFailure());
			}

			CurrentSessionSearchPromisesUserMap.Remove(Params.LocalAccountId);
			PendingV1SessionSearchesPerUser.Remove(Params.LocalAccountId);
		});

		TSharedRef<FOnlineSessionSearch>& PendingV1SessionSearch = PendingV1SessionSearchesPerUser.Emplace(Params.LocalAccountId, BuildV1SessionSearch(Params));

		SessionsInterface->FindSessions(*LocalAccountId, PendingV1SessionSearch);
	}

	return Future;
}

TOptional<FOnlineError> FSessionsOSSAdapter::CheckState(const FFindSessions::Params& Params) const
{
	if (TOptional<FOnlineError> BaseCheck = Super::CheckState(Params))
	{
		return BaseCheck;
	}

	const FUniqueNetIdRef LocalAccountId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId).ToSharedRef();
	if (!LocalAccountId->IsValid())
	{
		UE_LOGF(LogOnlineServices, Verbose, "[FSessionsOSSAdapter::FindSessionsImpl] UniqueId not found for LocalAccountId %ls", *ToLogString(Params.LocalAccountId));
		
		return TOptional<FOnlineError>(Errors::InvalidUser());
	}

	if (PendingV1SessionSearchesPerUser.Contains(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Verbose, "[FSessionsOSSAdapter::FindSessionsImpl] Session search already in progress for LocalAccountId %ls", *ToLogString(Params.LocalAccountId));
		
		return TOptional<FOnlineError>(Errors::AlreadyPending());
	}

	return TOptional<FOnlineError>();
}

TFuture<TOnlineResult<FStartMatchmaking>> FSessionsOSSAdapter::StartMatchmakingImpl(const FStartMatchmaking::Params& Params)
{
	TPromise<TOnlineResult<FStartMatchmaking>> Promise;
	TFuture<TOnlineResult<FStartMatchmaking>> Future = Promise.GetFuture();

	FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

	TArray<FSessionMatchmakingUser> MatchMakingUsers;
	FSessionMatchmakingUser NewMatchMakingUser { ServicesOSSAdapter.GetAccountIdRegistry().GetIdValue(Params.SessionCreationParameters.LocalAccountId).ToSharedRef() };
	MatchMakingUsers.Add(NewMatchMakingUser);

	FOnlineSessionSettings NewSessionSettings = BuildV1SettingsForCreate(Params.SessionCreationParameters);

	/** We build a mock params struct for FindSessions and convert it to a V1 session search */
	FFindSessions::Params FindSessionsParams;
	FindSessionsParams.bFindLANSessions = false;
	FindSessionsParams.Filters = Params.SessionSearchFilters;
	FindSessionsParams.LocalAccountId = Params.SessionCreationParameters.LocalAccountId;
	FindSessionsParams.MaxResults = 1;

	TSharedRef<FOnlineSessionSearch>& SessionSearch = PendingV1SessionSearchesPerUser.Emplace(Params.SessionCreationParameters.LocalAccountId, BuildV1SessionSearch(FindSessionsParams));

	SessionsInterface->StartMatchmaking(MatchMakingUsers, Params.SessionCreationParameters.SessionName, NewSessionSettings, SessionSearch, *MakeDelegateAdapter(this, [this, Promise = MoveTemp(Promise), Params](FName SessionName, const ::FOnlineError& ErrorDetails, const FSessionMatchmakingResults& Results) mutable
	{
		if (!ErrorDetails.bSucceeded)
		{
			Promise.EmplaceValue(Errors::FromOssError(ErrorDetails));
			return;
		}

		TSharedRef<FSessionCommon> V2Session = BuildV2Session(SessionsInterface->GetNamedSession(SessionName));

		AddSessionWithReferences(V2Session, Params.SessionCreationParameters.SessionName, Params.SessionCreationParameters.LocalAccountId, Params.SessionCreationParameters.bPresenceEnabled);

		Promise.EmplaceValue(FStartMatchmaking::Result{ });

		FSessionJoined Event{ { Params.SessionCreationParameters.LocalAccountId }, V2Session->GetSessionId()};

		SessionEvents.OnSessionJoined.Broadcast(Event);
	}));

	return Future;
}

TFuture<TOnlineResult<FJoinSession>> FSessionsOSSAdapter::JoinSessionImpl(const FJoinSession::Params& Params)
{
	TPromise<TOnlineResult<FJoinSession>> Promise;
	TFuture<TOnlineResult<FJoinSession>> Future = Promise.GetFuture();

	MakeMulticastAdapter(this, SessionsInterface->OnJoinSessionCompleteDelegates,
	[this, Promise = MoveTemp(Promise), Params](FName SessionName, EOnJoinSessionCompleteResult::Type Result) mutable
	{
		if (Result != EOnJoinSessionCompleteResult::Success)
		{
			Promise.EmplaceValue(Errors::RequestFailure());
			return;
		}

		TOnlineResult<FGetSessionById> GetSessionByIdResult = GetSessionById({ Params.SessionId });
		if (GetSessionByIdResult.IsError())
		{
			// If no result is found, the id might be expired, which we should notify
			if (GetSessionIdRegistry().IsHandleExpired(Params.SessionId))
			{
				UE_LOGF(LogOnlineServices, Warning, "[%ls] SessionId parameter [%ls] is expired. Please call FindSessions to get an updated list of available sessions ", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionId));
			}

			Promise.EmplaceValue(MoveTemp(GetSessionByIdResult.GetErrorValue()));
			return;
		}

		TSharedRef<const ISession> FoundSession = GetSessionByIdResult.GetOkValue().Session;

		AddSessionReferences(FoundSession->GetSessionId(), Params.SessionName, Params.LocalAccountId, Params.bPresenceEnabled);

		// After successfully joining a session, we'll remove all related invites if any are found
		ClearSessionInvitesForSession(Params.LocalAccountId, FoundSession->GetSessionId());

		Promise.EmplaceValue(FJoinSession::Result{ });

		FSessionJoined Event{ { Params.LocalAccountId }, FoundSession->GetSessionId() };

		SessionEvents.OnSessionJoined.Broadcast(Event);
	});

	TOnlineResult<FGetSessionById> GetSessionByIdResult = GetSessionById({ Params.SessionId });
	if (GetSessionByIdResult.IsError())
	{
		Promise.EmplaceValue(GetSessionByIdResult.GetErrorValue());
		return Future;
	}

	const TSharedRef<const ISession>& FoundSession = GetSessionByIdResult.GetOkValue().Session;

	const FUniqueNetIdPtr LocalAccountIdPtr = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
	check(LocalAccountIdPtr.IsValid());
	const FUniqueNetIdRef LocalAccountId = LocalAccountIdPtr.ToSharedRef();

	FOnlineSessionSearchResult SearchResult;

	if (const FCustomSessionSetting* PingInMs = FoundSession->GetSessionSettings().CustomSettings.Find(OSS_ADAPTER_SESSIONS_PING_IN_MS))
	{
		SearchResult.PingInMs = (int32)PingInMs->Data.GetInt64();
	}

	SearchResult.Session = BuildV1Session(FoundSession);

	SessionsInterface->JoinSession(*LocalAccountId, Params.SessionName, SearchResult);

	return Future;
}

TOptional<FOnlineError> FSessionsOSSAdapter::CheckState(const FJoinSession::Params& Params) const
{
	if (TOptional<FOnlineError> BaseCheck = Super::CheckState(Params))
	{
		return BaseCheck;
	}

	const FUniqueNetIdRef LocalAccountId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId).ToSharedRef();
	if (!LocalAccountId->IsValid())
	{
		UE_LOGF(LogOnlineServices, Verbose, "[%ls] UniqueId not found for LocalAccountId %ls", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.LocalAccountId));
		
		return TOptional<FOnlineError>(Errors::InvalidUser());
	}

	return TOptional<FOnlineError>();
}

TFuture<TOnlineResult<FAddSessionMember>> FSessionsOSSAdapter::AddSessionMemberImpl(const FAddSessionMember::Params& Params)
{
	TPromise<TOnlineResult<FAddSessionMember>> Promise;
	TFuture<TOnlineResult<FAddSessionMember>> Future = Promise.GetFuture();

	MakeMulticastAdapter(this, SessionsInterface->OnRegisterPlayersCompleteDelegates,
		[this, Promise = MoveTemp(Promise), Params](FName SessionName, const TArray<FUniqueNetIdRef>& Players, bool bWasSuccessful) mutable
		{
			if (bWasSuccessful)
			{
				Promise.EmplaceValue(FAddSessionMember::Result{ });
			}
			else
			{
				Promise.EmplaceValue(Errors::RequestFailure());
			}
		});

	FAuthOSSAdapter* Auth = Services.Get<FAuthOSSAdapter>();
	TArray<FUniqueNetIdRef> TargetUserNetIds;		
	FUniqueNetIdRef TargetUserNetId = Auth->GetUniqueNetId(Params.LocalAccountId).ToSharedRef();
	if (TargetUserNetId->IsValid())
	{
		TargetUserNetIds.Add(TargetUserNetId);
	}	

	SessionsInterface->RegisterPlayers(Params.SessionName, TargetUserNetIds);

	return Future;
}

TFuture<TOnlineResult<FRemoveSessionMember>> FSessionsOSSAdapter::RemoveSessionMemberImpl(const FRemoveSessionMember::Params& Params)
{
	TPromise<TOnlineResult<FRemoveSessionMember>> Promise;
	TFuture<TOnlineResult<FRemoveSessionMember>> Future = Promise.GetFuture();

	MakeMulticastAdapter(this, SessionsInterface->OnUnregisterPlayersCompleteDelegates,
		[this, Promise = MoveTemp(Promise)](FName SessionName, const TArray<FUniqueNetIdRef>& Players, bool bWasSuccessful) mutable
		{
			if (bWasSuccessful)
			{
				Promise.EmplaceValue(FRemoveSessionMember::Result{ });
			}
			else
			{
				Promise.EmplaceValue(Errors::RequestFailure());
			}
		});

	FAuthOSSAdapter* Auth = Services.Get<FAuthOSSAdapter>();
	TArray<FUniqueNetIdRef> TargetUserNetIds;
	FUniqueNetIdRef TargetUserNetId = Auth->GetUniqueNetId(Params.LocalAccountId).ToSharedRef();
	if (TargetUserNetId->IsValid())
	{
		TargetUserNetIds.Add(TargetUserNetId);
	}

	SessionsInterface->UnregisterPlayers(Params.SessionName, TargetUserNetIds);

	return Future;
}

TFuture<TOnlineResult<FSendSessionInvite>> FSessionsOSSAdapter::SendSessionInviteImpl(const FSendSessionInvite::Params& Params)
{
	TPromise<TOnlineResult<FSendSessionInvite>> Promise;
	TFuture<TOnlineResult<FSendSessionInvite>> Future = Promise.GetFuture();

	FAuthOSSAdapter* Auth = Services.Get<FAuthOSSAdapter>();

	const FUniqueNetIdPtr LocalAccountIdPtr = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
	check(LocalAccountIdPtr.IsValid());
	const FUniqueNetIdRef LocalAccountId = LocalAccountIdPtr.ToSharedRef();

	TArray<FUniqueNetIdRef> TargetUserNetIds;
	for (const FAccountId& TargetUser : Params.TargetUsers)
	{
		FUniqueNetIdRef TargetUserNetId = Auth->GetUniqueNetId(TargetUser).ToSharedRef();
		if (TargetUserNetId->IsValid())
		{
			TargetUserNetIds.Add(TargetUserNetId);
		}
	}

	SessionsInterface->SendSessionInviteToFriends(*LocalAccountId, Params.SessionName, TargetUserNetIds);

	Promise.EmplaceValue(FSendSessionInvite::Result{ });

	return Future;
}

TOptional<FOnlineError> FSessionsOSSAdapter::CheckState(const FSendSessionInvite::Params& Params) const
{
	if (TOptional<FOnlineError> BaseCheck = Super::CheckState(Params))
	{
		return BaseCheck;
	}

	const FUniqueNetIdRef LocalAccountId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId).ToSharedRef();
	if (!LocalAccountId->IsValid())
	{
		UE_LOGF(LogOnlineServices, Verbose, "[%ls] UniqueId not found for LocalAccountId %ls", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.LocalAccountId));

		return TOptional<FOnlineError>(Errors::InvalidUser());
	}

	return TOptional<FOnlineError>();
}

TOnlineResult<FGetResolvedConnectString> FSessionsOSSAdapter::GetResolvedConnectString(const FGetResolvedConnectString::Params& Params)
{
	if (Params.SessionId.IsValid())
	{
		TOnlineResult<FGetSessionById> Result = GetSessionById({ Params.SessionId });
		if (Result.IsOk())
		{
			FOnlineSessionSearchResult SearchResult;
			SearchResult.Session = BuildV1Session(Result.GetOkValue().Session);

			FString ConnectString;

			if (SessionsInterface->GetResolvedConnectString(SearchResult, Params.PortType, ConnectString))
			{
				return TOnlineResult<FGetResolvedConnectString>({ ConnectString });
			}
			else
			{
				return TOnlineResult<FGetResolvedConnectString>(Errors::RequestFailure());
			}
		}
		else
		{
			return TOnlineResult<FGetResolvedConnectString>(Result.GetErrorValue());
		}
	}
	else
	{
		// No valid session id set
		UE_LOGF(LogOnlineServices, Verbose, "[FSessionsOSSAdapter::GetResolvedConnectString] Invalid SessionId %ls", *ToLogString(Params.SessionId));
		return TOnlineResult<FGetResolvedConnectString>(Errors::InvalidParams());
	}
}

bool FSessionsOSSAdapter::UpdateV2Session(const FName& SessionName)
{
	bool bResult = false;

	if (FNamedOnlineSession* NamedOnlineSession = SessionsInterface->GetNamedSession(SessionName))
	{
		TOnlineResult<FGetMutableSessionByName> GetMutableSessionByNameResult = GetMutableSessionByName({ SessionName });
		if (GetMutableSessionByNameResult.IsOk())
		{
			// We update our local session
			TSharedRef<FSessionCommon>& FoundSession = GetMutableSessionByNameResult.GetOkValue().Session;

			FoundSession = BuildV2Session(NamedOnlineSession);

			bResult = true;
		}
	}

	return bResult;
}

FOnlineSessionSettings FSessionsOSSAdapter::BuildV1SettingsForCreate(const FCreateSession::Params& Params) const
{
	FOnlineSessionSettings Result;

	Result.bAllowInvites = Params.SessionSettings.bAllowNewMembers;
	Result.bAllowJoinInProgress = Params.SessionSettings.bAllowNewMembers;
	Result.bAllowJoinViaPresence = Params.SessionSettings.bAllowNewMembers;
	Result.bAllowJoinViaPresenceFriendsOnly = Params.SessionSettings.bAllowNewMembers;
	Result.bAntiCheatProtected = Params.bAntiCheatProtected;
	Result.bIsDedicated = IsRunningDedicatedServer();
	Result.bIsLANMatch = Params.bIsLANSession;
	Result.bShouldAdvertise = Params.SessionSettings.bAllowNewMembers;
	if (const FCustomSessionSetting* BuildUniqueId = Params.SessionSettings.CustomSettings.Find(OSS_ADAPTER_SESSIONS_BUILD_UNIQUE_ID))
	{
		Result.BuildUniqueId = BuildUniqueId->Data.GetInt64();
	}
	if (const FCustomSessionSetting* UseLobbiesIfAvailable = Params.SessionSettings.CustomSettings.Find(OSS_ADAPTER_SESSIONS_USE_LOBBIES_IF_AVAILABLE))
	{
		Result.bUseLobbiesIfAvailable = UseLobbiesIfAvailable->Data.GetBoolean();
	}
	if (const FCustomSessionSetting* UseLobbiesVoiceChatIfAvailable = Params.SessionSettings.CustomSettings.Find(OSS_ADAPTER_SESSIONS_USE_LOBBIES_VOICE_CHAT_IF_AVAILABLE))
	{
		Result.bUseLobbiesVoiceChatIfAvailable = UseLobbiesVoiceChatIfAvailable->Data.GetBoolean();
	}

	Result.bUsesPresence = Params.bPresenceEnabled;

	if (const FCustomSessionSetting* UsesStats = Params.SessionSettings.CustomSettings.Find(OSS_ADAPTER_SESSIONS_USES_STATS))
	{
		Result.bUsesStats = UsesStats->Data.GetBoolean();
	}
	Result.NumPrivateConnections = Params.SessionSettings.NumMaxConnections;
	Result.NumPublicConnections = Params.SessionSettings.NumMaxConnections;
	Result.SessionIdOverride = Params.SessionIdOverride;

	Result.Settings = GetV1SessionSettings(Params.SessionSettings.CustomSettings);
	Result.Settings.Add(OSS_ADAPTER_SESSIONS_SCHEMA_NAME, Params.SessionSettings.SchemaName.ToString());
	Result.Settings.Add(SETTING_SESSION_TEMPLATE_NAME, Params.SessionSettings.SchemaName.ToString());
	Result.Settings.Add(OSS_ADAPTER_SESSIONS_ALLOW_SANCTIONED_PLAYERS, Params.bAllowSanctionedPlayers);

	FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

	Result.MemberSettings.Add(ServicesOSSAdapter.GetAccountIdRegistry().GetIdValue(Params.LocalAccountId).ToSharedRef());

	return Result;
}

FOnlineSessionSettings FSessionsOSSAdapter::BuildV1SettingsForUpdate(const FAccountId& LocalAccountId, const TSharedRef<ISession>& Session, const FSessionSettings& SessionSettings) const
{
	FOnlineSessionSettings Result;

	Result.bAllowInvites = SessionSettings.bAllowNewMembers;
	Result.bAllowJoinInProgress = SessionSettings.bAllowNewMembers;
	Result.bAllowJoinViaPresence = SessionSettings.bAllowNewMembers;
	Result.bAllowJoinViaPresenceFriendsOnly = SessionSettings.bAllowNewMembers;
	Result.Settings.Add(OSS_ADAPTER_SESSIONS_ALLOW_SANCTIONED_PLAYERS, Session->GetSessionInfo().bAllowSanctionedPlayers);
	Result.bAntiCheatProtected = Session->GetSessionInfo().bAntiCheatProtected;
	Result.bIsDedicated = Session->GetSessionInfo().bIsDedicatedServerSession;
	Result.bIsLANMatch = Session->GetSessionInfo().bIsLANSession;
	Result.bShouldAdvertise = SessionSettings.bAllowNewMembers;
	if (const FCustomSessionSetting* BuildUniqueId = SessionSettings.CustomSettings.Find(OSS_ADAPTER_SESSIONS_BUILD_UNIQUE_ID))
	{
		Result.BuildUniqueId = BuildUniqueId->Data.GetInt64();
	}
	if (const FCustomSessionSetting* BuildUniqueId = SessionSettings.CustomSettings.Find(OSS_ADAPTER_SESSIONS_USE_LOBBIES_IF_AVAILABLE))
	{
		Result.bUseLobbiesIfAvailable = BuildUniqueId->Data.GetBoolean();
	}
	if (const FCustomSessionSetting* BuildUniqueId = SessionSettings.CustomSettings.Find(OSS_ADAPTER_SESSIONS_USE_LOBBIES_VOICE_CHAT_IF_AVAILABLE))
	{
		Result.bUseLobbiesVoiceChatIfAvailable = BuildUniqueId->Data.GetBoolean();
	}

	Result.bUsesPresence = false;
	TOnlineResult<FIsPresenceSession> IsPresenceSessionResult = IsPresenceSession({ LocalAccountId, Session->GetSessionId() });
	if (IsPresenceSessionResult.IsOk())
	{
		Result.bUsesPresence = IsPresenceSessionResult.GetOkValue().bIsPresenceSession;
	}

	if (const FCustomSessionSetting* BuildUniqueId = SessionSettings.CustomSettings.Find(OSS_ADAPTER_SESSIONS_USES_STATS))
	{
		Result.bUsesStats = BuildUniqueId->Data.GetBoolean();
	}
	Result.NumPrivateConnections = SessionSettings.NumMaxConnections;
	Result.NumPublicConnections = SessionSettings.NumMaxConnections;
	Result.SessionIdOverride = Session->GetSessionInfo().SessionIdOverride;

	Result.Settings = GetV1SessionSettings(SessionSettings.CustomSettings);
	Result.Settings.Add(OSS_ADAPTER_SESSIONS_SCHEMA_NAME, SessionSettings.SchemaName.ToString());
	Result.Settings.Add(SETTING_SESSION_TEMPLATE_NAME, SessionSettings.SchemaName.ToString());
	
	FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

	for (const FAccountId& SessionMember : Session->GetSessionMembers())
	{
		Result.MemberSettings.Add(ServicesOSSAdapter.GetAccountIdRegistry().GetIdValue(SessionMember).ToSharedRef());
	}

	return Result;
}

void FSessionsOSSAdapter::WriteV2SessionSettingsFromV1Session(const FOnlineSession* InSession, TSharedRef<FSessionOSSAdapter>& OutSession) const
{
	OutSession->SessionSettings.NumMaxConnections = FMath::Max(InSession->SessionSettings.NumPrivateConnections, InSession->SessionSettings.NumPublicConnections);

	FString SchemaNameStr;
	InSession->SessionSettings.Get(OSS_ADAPTER_SESSIONS_SCHEMA_NAME, SchemaNameStr);
	OutSession->SessionSettings.SchemaName = FSchemaId(SchemaNameStr);

	OutSession->SessionSettings.CustomSettings = GetV2SessionSettings(InSession->SessionSettings.Settings);

	FCustomSessionSetting BuildUniqueId;
	BuildUniqueId.Data.Set((int64)InSession->SessionSettings.BuildUniqueId);
	OutSession->SessionSettings.CustomSettings.Add(OSS_ADAPTER_SESSIONS_BUILD_UNIQUE_ID, BuildUniqueId);

	FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

	for (const TPair<FUniqueNetIdRef, ::FSessionSettings>& Entry : InSession->SessionSettings.MemberSettings)
	{
		FAccountId SessionMemberId = ServicesOSSAdapter.GetAccountIdRegistry().FindOrAddHandle(Entry.Key);

		OutSession->SessionMembers.Emplace(SessionMemberId);
	}
}

void FSessionsOSSAdapter::WriteV2SessionSettingsFromV1NamedSession(const FNamedOnlineSession* InSession, TSharedRef<FSessionOSSAdapter>& OutSession) const
{
	bool bPublicJoinable, bFriendJoinable, bInviteOnly, bAllowInvites;
	InSession->GetJoinability(bPublicJoinable, bFriendJoinable, bInviteOnly, bAllowInvites);

	OutSession->SessionSettings.bAllowNewMembers = bPublicJoinable || bFriendJoinable || bInviteOnly;
	OutSession->SessionSettings.JoinPolicy = bInviteOnly ? ESessionJoinPolicy::InviteOnly : bFriendJoinable ? ESessionJoinPolicy::FriendsOnly : ESessionJoinPolicy::Public; // We use Public as the default as closed sessions will have bAllowNewMembers as false

	FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

	// We'll add all the Registered users as SessionMembers with empty data for now
	for (const FUniqueNetIdRef& RegisteredPlayer : InSession->RegisteredPlayers)
	{
		OutSession->SessionMembers.Emplace(ServicesOSSAdapter.GetAccountIdRegistry().FindOrAddHandle(RegisteredPlayer));
	}
}

FOnlineSession FSessionsOSSAdapter::BuildV1Session(const TSharedRef<const ISession> InSession) const
{
	return FOnlineSession(FSessionOSSAdapter::Cast(*InSession).GetV1Session());
}

TSharedRef<FSessionCommon> FSessionsOSSAdapter::BuildV2Session(const FNamedOnlineSession* InSession) const
{
	FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

	TSharedRef<FSessionOSSAdapter> Session = MakeShared<FSessionOSSAdapter>(*InSession, GetSessionIdRegistry().FindOrAddHandle(InSession->SessionInfo->GetSessionId().AsShared()));

	// Prefer LocalOwnerId (set by EOS, GDK, Steam), fall back to OwningUserId (set by all V1 subsystems including Null)
	if (InSession->LocalOwnerId)
	{
		Session->OwnerAccountId = ServicesOSSAdapter.GetAccountIdRegistry().FindOrAddHandle(InSession->LocalOwnerId->AsShared());
	}
	else if (InSession->OwningUserId)
	{
		Session->OwnerAccountId = ServicesOSSAdapter.GetAccountIdRegistry().FindOrAddHandle(InSession->OwningUserId->AsShared());
	}

	WriteV2SessionSettingsFromV1NamedSession(InSession, Session);

	return Session;
}

TSharedRef<FSessionCommon> FSessionsOSSAdapter::BuildV2Session(const FOnlineSession* InSession) const
{
	FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);

	TSharedRef<FSessionOSSAdapter> Session = MakeShared<FSessionOSSAdapter>(*InSession, GetSessionIdRegistry().FindOrAddHandle(InSession->SessionInfo->GetSessionId().AsShared()));

	WriteV2SessionSettingsFromV1Session(InSession, Session);

	return Session;
}

TArray<TSharedRef<FSessionCommon>> FSessionsOSSAdapter::BuildV2SessionSearchResults(const TArray<FOnlineSessionSearchResult>& SessionSearchResults) const
{
	TArray<TSharedRef<FSessionCommon>> FoundSessions;

	for (const FOnlineSessionSearchResult& SearchResult : SessionSearchResults)
	{
		TSharedRef<FSessionCommon> FoundSession = BuildV2Session(&SearchResult.Session);

		FCustomSessionSetting PingInMs;
		PingInMs.Data.Set((int64)SearchResult.PingInMs);
		FoundSession->SessionSettings.CustomSettings.Emplace(OSS_ADAPTER_SESSIONS_PING_IN_MS, PingInMs);

		FoundSessions.Add(FoundSession);
	}

	return FoundSessions;
}

bool FSessionsOSSAdapter::GetAccountIdAndSessionIdFromNamedSession(const FName& SessionName, FAccountId& AccountId, FOnlineSessionId& OnlineSessionId) const
{
	if (GetV2SessionCreatorFromV1NamedSession(SessionName, AccountId))
	{
		if (GetV2SessionIdFromV1NamedSession(SessionName, OnlineSessionId))
		{
			return true;
		}
	}

	return false;
}

bool FSessionsOSSAdapter::GetV2SessionIdFromV1NamedSession(const FName& SessionName, FOnlineSessionId& OnlineSessionId) const
{
	const FNamedOnlineSession* NamedOnlineSession = SessionsInterface->GetNamedSession(SessionName);
	if (NamedOnlineSession)
	{
		const FUniqueNetIdRef SessionUniqueNetIdRef = NamedOnlineSession->SessionInfo->GetSessionId().AsShared();
		OnlineSessionId = GetSessionIdRegistry().FindOrAddHandle(SessionUniqueNetIdRef);
		return true;
	}

	return false;
}

bool FSessionsOSSAdapter::GetV2SessionCreatorFromV1NamedSession(const FName& SessionName, FAccountId& AccountId) const
{
	const FNamedOnlineSession* NamedOnlineSession = SessionsInterface->GetNamedSession(SessionName);
	if (NamedOnlineSession)
	{
		const FUniqueNetIdPtr LocalOwnerIdPtr = NamedOnlineSession->OwningUserId;
		if (LocalOwnerIdPtr.IsValid())
		{
			const FOnlineServicesOSSAdapter& ServicesOSSAdapter = static_cast<FOnlineServicesOSSAdapter&>(Services);
			AccountId = ServicesOSSAdapter.GetAccountIdRegistry().FindOrAddHandle(LocalOwnerIdPtr->AsShared());
			if (AccountId.IsValid())
			{
				return true;
			}
		}
	}

	return false;
}

FOnlineSessionIdRegistryOSSAdapter& FSessionsOSSAdapter::GetSessionIdRegistry() const
{
	FOnlineSessionIdRegistryOSSAdapter* SessionIdRegistry = static_cast<FOnlineSessionIdRegistryOSSAdapter*>(FOnlineIdRegistryRegistry::Get().GetSessionIdRegistry(Services.GetServicesProvider()));
	check(SessionIdRegistry);
	return *SessionIdRegistry;
}

FOnlineSessionInviteIdRegistryOSSAdapter& FSessionsOSSAdapter::GetSessionInviteIdRegistry() const
{
	FOnlineSessionInviteIdRegistryOSSAdapter* SessionInviteIdRegistry = static_cast<FOnlineSessionInviteIdRegistryOSSAdapter*>(FOnlineIdRegistryRegistry::Get().GetSessionInviteIdRegistry(Services.GetServicesProvider()));
	check(SessionInviteIdRegistry);
	return *SessionInviteIdRegistry;
}

/* UE::Online */ }