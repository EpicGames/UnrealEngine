// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SocialOSSAdapter.h"

#include "Online/OnlineServicesOSSAdapter.h"
#include "Online/DelegateAdapter.h"
#include "Online/ErrorsOSSAdapter.h"

#include "OnlineSubsystem.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlineIdentityInterface.h"

namespace UE::Online {

ERelationship InviteStatusToRelationship(EInviteStatus::Type Status)
{
	switch (Status)
	{
		default:
		case EInviteStatus::Accepted:			return ERelationship::Friend;
		case EInviteStatus::PendingInbound:		return ERelationship::InviteSent;
		case EInviteStatus::PendingOutbound:	return ERelationship::InviteReceived;
		case EInviteStatus::Blocked:			return ERelationship::Blocked;
		case EInviteStatus::Unknown:			return ERelationship::NotFriend;
	}
}

void FSocialOSSAdapter::PostInitialize()
{
	Auth = Services.Get<FAuthOSSAdapter>();
	FriendsInt = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem().GetFriendsInterface();

	for (int LocalPlayerNum = 0; LocalPlayerNum < MAX_LOCAL_PLAYERS; ++LocalPlayerNum)
	{
		OnFriendsChangedDelegateHandles[LocalPlayerNum] = FriendsInt->OnFriendsChangeDelegates[LocalPlayerNum].AddThreadSafeSP(this, &FSocialOSSAdapter::OnFriendsChanged);
		OnBlockListChangedDelegateHandles[LocalPlayerNum] = FriendsInt->OnBlockListChangeDelegates[LocalPlayerNum].AddThreadSafeSP(this, &FSocialOSSAdapter::OnBlockListChanged);
	}
}

void FSocialOSSAdapter::PreShutdown()
{
	for (int LocalPlayerNum = 0; LocalPlayerNum < MAX_LOCAL_PLAYERS; ++LocalPlayerNum)
	{
		FriendsInt->OnFriendsChangeDelegates[LocalPlayerNum].Remove(OnFriendsChangedDelegateHandles[LocalPlayerNum]);
		FriendsInt->OnBlockListChangeDelegates[LocalPlayerNum].Remove(OnBlockListChangedDelegateHandles[LocalPlayerNum]);
	}
}

TOnlineAsyncOpHandle<FQueryFriends> FSocialOSSAdapter::QueryFriends(FQueryFriends::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryFriends> Op = GetJoinableOp<FQueryFriends>(MoveTemp(Params));

	if (Op->IsReady())
	{
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FQueryFriends>& Op)
	{
		int32 LocalUserNum = Auth->GetLocalUserNum(Op.GetParams().LocalAccountId);

		if (LocalUserNum != INDEX_NONE)
		{
			const bool bStarted = FriendsInt->ReadFriendsList(LocalUserNum, TEXT(""), *MakeDelegateAdapter(this,
				[this, WeakOp = Op.AsWeak()](int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr)
				{
					if (TSharedPtr<TOnlineAsyncOp<FQueryFriends>> PinnedOp = WeakOp.Pin())
					{
						if (bWasSuccessful)
						{
							TArray<TSharedRef<FOnlineFriend>> OSSFriends;
							if (FriendsInt->GetFriendsList(LocalUserNum, FString(), OSSFriends))
							{
								TMap<FAccountId, ERelationship> FriendRelationships;
								for (TSharedRef<FOnlineFriend>& Friend : OSSFriends)
								{
									FriendRelationships.Emplace(Auth->GetAccountId(Friend->GetUserId()), InviteStatusToRelationship(Friend->GetInviteStatus()));
								}
								Friends.FindOrAdd(PinnedOp->GetParams().LocalAccountId) = MoveTemp(FriendRelationships);
							}

							PinnedOp->SetResult({});
						}
						else
						{
							PinnedOp->SetError(Errors::FromOssErrorCode(ErrorStr));
						}
					}
				}));

			if (!bStarted)
			{
				Op.SetError(Errors::Unknown());
			}
		}
		else
		{
			Op.SetError(Errors::InvalidParams());
		}
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineResult<FGetFriends> FSocialOSSAdapter::GetFriends(FGetFriends::Params&& Params)
{
	int32 LocalUserNum = Auth->GetLocalUserNum(Params.LocalAccountId);

	if (LocalUserNum != INDEX_NONE)
	{
		FGetFriends::Result Result;

		TArray<TSharedRef<FOnlineFriend>> OSSFriends;
		if (FriendsInt->GetFriendsList(LocalUserNum, FString(), OSSFriends))
		{
			Result.Friends.Reserve(OSSFriends.Num());
			for (TSharedRef<FOnlineFriend> Friend : OSSFriends)
			{
				TSharedRef<FFriend> ResultFriend = MakeShared<FFriend>();
				ResultFriend->FriendId = Auth->GetAccountId(Friend->GetUserId());
				ResultFriend->DisplayName = Friend->GetDisplayName();
				ResultFriend->Nickname = Friend->GetDisplayName();
				ResultFriend->Relationship = InviteStatusToRelationship(Friend->GetInviteStatus());

				Result.Friends.Emplace(MoveTemp(ResultFriend));
			}
			return TOnlineResult<FGetFriends>(Result);
		}
		else
		{
			return TOnlineResult<FGetFriends>(Errors::InvalidUser());
		}
	}
	else
	{
		return TOnlineResult<FGetFriends>(Errors::InvalidUser());
	}
}

TOnlineAsyncOpHandle<FSendFriendInvite> FSocialOSSAdapter::SendFriendInvite(FSendFriendInvite::Params&& Params)
{
	TOnlineAsyncOpRef<FSendFriendInvite> Op = GetOp<FSendFriendInvite>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FSendFriendInvite>& Op)
	{
		int32 LocalUserNum = Auth->GetLocalUserNum(Op.GetParams().LocalAccountId);
		FUniqueNetIdPtr FriendId = Auth->GetUniqueNetId(Op.GetParams().TargetAccountId);

		if (LocalUserNum == INDEX_NONE || !FriendId)
		{
			Op.SetError(Errors::InvalidParams());
			return;
		}

		const bool bStarted = FriendsInt->SendInvite(LocalUserNum, *FriendId, FString(), *MakeDelegateAdapter(this,
			[WeakOp = Op.AsWeak()](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& FriendId, const FString& ListName, const FString& ErrorStr)
			{
				if (TSharedPtr<TOnlineAsyncOp<FSendFriendInvite>> PinnedOp = WeakOp.Pin())
				{
					if (bWasSuccessful)
					{
						PinnedOp->SetResult({});
					}
					else
					{
						PinnedOp->SetError(Errors::FromOssErrorCode(ErrorStr));
					}
				}
			}));

		if (!bStarted)
		{
			Op.SetError(Errors::Unknown());
		}
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAcceptFriendInvite> FSocialOSSAdapter::AcceptFriendInvite(FAcceptFriendInvite::Params&& Params)
{
	TOnlineAsyncOpRef<FAcceptFriendInvite> Op = GetOp<FAcceptFriendInvite>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FAcceptFriendInvite>& Op)
	{
		int32 LocalUserNum = Auth->GetLocalUserNum(Op.GetParams().LocalAccountId);
		FUniqueNetIdPtr FriendId = Auth->GetUniqueNetId(Op.GetParams().TargetAccountId);

		if (LocalUserNum == INDEX_NONE || !FriendId)
		{
			Op.SetError(Errors::InvalidParams());
			return;
		}

		const bool bStarted = FriendsInt->AcceptInvite(LocalUserNum, *FriendId, FString(), *MakeDelegateAdapter(this,
			[WeakOp = Op.AsWeak()](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& FriendId, const FString& ListName, const FString& ErrorStr)
			{
				if (TSharedPtr<TOnlineAsyncOp<FAcceptFriendInvite>> PinnedOp = WeakOp.Pin())
				{
					if (bWasSuccessful)
					{
						PinnedOp->SetResult({});
					}
					else
					{
						PinnedOp->SetError(Errors::FromOssErrorCode(ErrorStr));
					}
				}
			}));

		if (!bStarted)
		{
			Op.SetError(Errors::Unknown());
		}
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FRejectFriendInvite> FSocialOSSAdapter::RejectFriendInvite(FRejectFriendInvite::Params&& Params)
{
	TOnlineAsyncOpRef<FRejectFriendInvite> Op = GetOp<FRejectFriendInvite>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FRejectFriendInvite>& Op)
	{
		int32 LocalUserNum = Auth->GetLocalUserNum(Op.GetParams().LocalAccountId);
		FUniqueNetIdPtr FriendId = Auth->GetUniqueNetId(Op.GetParams().TargetAccountId);

		if (LocalUserNum == INDEX_NONE || !FriendId)
		{
			Op.SetError(Errors::InvalidParams());
			return;
		}

		MakeMulticastAdapter(this, FriendsInt->OnRejectInviteCompleteDelegates[LocalUserNum],
			[WeakOp = Op.AsWeak()](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& FriendId, const FString& ListName, const FString& ErrorStr)
			{
				if (TSharedPtr<TOnlineAsyncOp<FRejectFriendInvite>> PinnedOp = WeakOp.Pin())
				{
					if (bWasSuccessful)
					{
						PinnedOp->SetResult({});
					}
					else
					{
						PinnedOp->SetError(Errors::FromOssErrorCode(ErrorStr));
					}
				}
			});

		if (!FriendsInt->RejectInvite(LocalUserNum, *FriendId, FString()))
		{
			Op.SetError(Errors::Unknown());
		}
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FQueryBlockedUsers> FSocialOSSAdapter::QueryBlockedUsers(FQueryBlockedUsers::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryBlockedUsers> Op = GetJoinableOp<FQueryBlockedUsers>(MoveTemp(Params));

	if (Op->IsReady())
	{
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FQueryBlockedUsers>& Op)
	{
		FUniqueNetIdPtr LocalUserId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);

		if (LocalUserId)
		{
			MakeMulticastAdapter(this, FriendsInt->OnQueryBlockedPlayersCompleteDelegates,
				[WeakOp = Op.AsWeak()](const FUniqueNetId& UserId, bool bWasSuccessful, const FString& Error)
				{
					if (TSharedPtr<TOnlineAsyncOp<FQueryBlockedUsers>> PinnedOp = WeakOp.Pin())
					{
						if (bWasSuccessful)
						{
							PinnedOp->SetResult({});
						}
						else
						{
							PinnedOp->SetError(Errors::FromOssErrorCode(Error));
						}
					}
				});

			if (!FriendsInt->QueryBlockedPlayers(*LocalUserId))
			{
				Op.SetError(Errors::Unknown());
			}
		}
		else
		{
			Op.SetError(Errors::InvalidUser());
		}
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineResult<FGetBlockedUsers> FSocialOSSAdapter::GetBlockedUsers(FGetBlockedUsers::Params&& Params)
{
	FUniqueNetIdPtr LocalUserId = Auth->GetUniqueNetId(Params.LocalAccountId);

	if (LocalUserId)
	{
		FGetBlockedUsers::Result Result;
		TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayers;
		if (FriendsInt->GetBlockedPlayers(*LocalUserId, BlockedPlayers))
		{
			Result.BlockedUsers.Reserve(BlockedPlayers.Num());
			for (TSharedRef<FOnlineBlockedPlayer> BlockedPlayer : BlockedPlayers)
			{
				Result.BlockedUsers.Emplace(Auth->GetAccountId(BlockedPlayer->GetUserId()));
			}
			return TOnlineResult<FGetBlockedUsers>(Result);
		}
		else
		{
			return TOnlineResult<FGetBlockedUsers>(Errors::InvalidUser());
		}
	}
	else
	{
		return TOnlineResult<FGetBlockedUsers>(Errors::InvalidUser());
	}
}

TOnlineAsyncOpHandle<FBlockUser> FSocialOSSAdapter::BlockUser(FBlockUser::Params&& Params)
{
	TOnlineAsyncOpRef<FBlockUser> Op = GetOp<FBlockUser>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FBlockUser>& Op)
	{
		int LocalPlayerNum = Auth->GetLocalUserNum(Op.GetParams().LocalAccountId);
		FUniqueNetIdPtr TargetAccountId = Auth->GetUniqueNetId(Op.GetParams().TargetAccountId);

		if (LocalPlayerNum != INDEX_NONE && TargetAccountId)
		{
			MakeMulticastAdapter(this, FriendsInt->OnBlockedPlayerCompleteDelegates[LocalPlayerNum],
				[WeakOp = Op.AsWeak()](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UniqueId, const FString& ListName, const FString& ErrorStr)
				{
					if (TSharedPtr<TOnlineAsyncOp<FBlockUser>> PinnedOp = WeakOp.Pin())
					{
						if (bWasSuccessful)
						{
							PinnedOp->SetResult({});
						}
						else
						{
							PinnedOp->SetError(Errors::FromOssErrorCode(ErrorStr));
						}
					}
				});

			if (!FriendsInt->BlockPlayer(LocalPlayerNum, *TargetAccountId))
			{
				Op.SetError(Errors::Unknown());
			}
		}
		else
		{
			Op.SetError(Errors::InvalidParams());
		}
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

FAccountId FSocialOSSAdapter::GetAccountId(int32 LocalUserNum)
{
	FUniqueNetIdPtr UniqueNetId = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem().GetIdentityInterface()->GetUniquePlayerId(LocalUserNum);
	if (!UniqueNetId)
	{
		return FAccountId();
	}
	return Auth->GetAccountId(UniqueNetId.ToSharedRef());
}

void FSocialOSSAdapter::OnFriendsChanged()
{
	// query and compare lists
	for (int32 LocalUserNum = 0; LocalUserNum < MAX_LOCAL_PLAYERS; ++LocalUserNum)
	{
		FAccountId LocalAccountId = GetAccountId(LocalUserNum);
		if (!LocalAccountId.IsValid())
		{
			continue;
		}

		TArray<TSharedRef<FOnlineFriend>> OSSFriends;
		if (FriendsInt->GetFriendsList(LocalUserNum, FString(), OSSFriends))
		{
			TMap<FAccountId, ERelationship>& FriendRelationships = Friends.FindOrAdd(LocalAccountId);
			TMap<FAccountId, ERelationship> NewFriendRelationships;
			for (TSharedRef<FOnlineFriend>& Friend : OSSFriends)
			{
				FAccountId FriendId = Auth->GetAccountId(Friend->GetUserId());
				ERelationship Relationship = InviteStatusToRelationship(Friend->GetInviteStatus());
				NewFriendRelationships.Emplace(FriendId, Relationship);

				if (ERelationship* OldRelationship = FriendRelationships.Find(FriendId))
				{
					if (Relationship != *OldRelationship)
					{
						// Relationship changed
						BroadcastRelationshipUpdated(LocalAccountId, FriendId, *OldRelationship, Relationship);
					}
				}
				else
				{
					// New friend
					BroadcastRelationshipUpdated(LocalAccountId, FriendId, ERelationship::NotFriend, Relationship);
				}
			}

			for (TPair<FAccountId, ERelationship> FriendRelationship : FriendRelationships)
			{
				if (!NewFriendRelationships.Contains(FriendRelationship.Key))
				{
					// No longer friends
					BroadcastRelationshipUpdated(LocalAccountId, FriendRelationship.Key, FriendRelationship.Value, ERelationship::NotFriend);
				}
			}

			FriendRelationships = MoveTemp(NewFriendRelationships);
		}
	}
}

void FSocialOSSAdapter::OnBlockListChanged(int32 LocalUserNum, const FString& ListName)
{
	// query and compare list
	FAccountId LocalAccountId = GetAccountId(LocalUserNum);
	if (!LocalAccountId.IsValid())
	{
		return;
	}

	FUniqueNetIdPtr UniqueNetId = Auth->GetUniqueNetId(LocalAccountId);
	if (!UniqueNetId)
	{
		return;
	}

	TSet<FAccountId>& OldBlockedUsers = BlockedUsers.FindOrAdd(LocalAccountId);
	TSet<FAccountId> NewBlockedUsers;
	TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayers;
	if (FriendsInt->GetBlockedPlayers(*UniqueNetId, BlockedPlayers))
	{
		for (const TSharedRef<FOnlineBlockedPlayer>& BlockedPlayer : BlockedPlayers)
		{
			FAccountId BlockedUserId = Auth->GetAccountId(BlockedPlayer->GetUserId());
			if (!OldBlockedUsers.Contains(BlockedUserId))
			{
				// Newly blocked user
				BroadcastRelationshipUpdated(LocalAccountId, BlockedUserId, ERelationship::NotFriend, ERelationship::Blocked);
			}
		}

		for (const FAccountId& BlockedUserId : OldBlockedUsers)
		{
			if (!NewBlockedUsers.Contains(BlockedUserId))
			{
				// Unblocked user
				BroadcastRelationshipUpdated(LocalAccountId, BlockedUserId, ERelationship::Blocked, ERelationship::NotFriend);
			}
		}
	}

	OldBlockedUsers = MoveTemp(NewBlockedUsers);
}

/* UE::Online */ }
