// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/StatsOSSAdapter.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/ErrorsOSSAdapter.h"

#include "Online/OnlineServicesOSSAdapter.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/DelegateAdapter.h"

#include "OnlineError.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"

namespace UE::Online {

void FStatsOSSAdapter::PostInitialize()
{
	Super::PostInitialize();

	Auth = Services.Get<FAuthOSSAdapter>();
	StatsInterface = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem().GetStatsInterface();
}

void FStatsOSSAdapter::ConvertStatValueV2ToStatUpdateV1(const FString& StatName, const FStatValue& StatValue, FOnlineStatUpdate& OutOnlineStatUpdate)
{
	FOnlineStatValue OnlineStatValue;

	if (StatValue.VariantData.IsType<double>())
	{
		OnlineStatValue.SetValue(StatValue.GetDouble());
	}
	else if (StatValue.VariantData.IsType<int64>())
	{
		OnlineStatValue.SetValue(StatValue.GetInt64());
	}
	else if (StatValue.VariantData.IsType<bool>())
	{
		OnlineStatValue.SetValue(StatValue.GetBoolean());
	}
	else if (StatValue.VariantData.IsType<FString>())
	{
		OnlineStatValue.SetValue(StatValue.GetString());
	}
	else
	{
		ensureMsgf(false, TEXT("Can't convert stat value!"));
	}

	FOnlineStatUpdate::EOnlineStatModificationType OnlineStatModificationType = FOnlineStatUpdate::EOnlineStatModificationType::Unknown;

	if (FStatDefinition* StatDefinition = StatDefinitions.Find(StatName))
	{
		switch (StatDefinition->ModifyMethod)
		{
		case EStatModifyMethod::Sum: OnlineStatModificationType = FOnlineStatUpdate::EOnlineStatModificationType::Sum; break;
		case EStatModifyMethod::Set: OnlineStatModificationType = FOnlineStatUpdate::EOnlineStatModificationType::Set; break;
		case EStatModifyMethod::Largest: OnlineStatModificationType = FOnlineStatUpdate::EOnlineStatModificationType::Largest; break;
		case EStatModifyMethod::Smallest: OnlineStatModificationType = FOnlineStatUpdate::EOnlineStatModificationType::Smallest; break;
		}
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "Can't find stat definition %ls", *StatName);
	}

	OutOnlineStatUpdate.Set(OnlineStatValue, OnlineStatModificationType);
}

void FStatsOSSAdapter::ConvertStatValueV1ToStatValueV2(const FOnlineStatValue& OnlineStatValue, FStatValue& OutStatValue)
{
	switch (OnlineStatValue.GetType())
	{
	case EOnlineKeyValuePairDataType::String: OnlineStatValue.GetValue(OutStatValue.VariantData.Get<FString>()); break;
	case EOnlineKeyValuePairDataType::Bool: OnlineStatValue.GetValue(OutStatValue.VariantData.Get<bool>()); break;
	case EOnlineKeyValuePairDataType::Double: OnlineStatValue.GetValue(OutStatValue.VariantData.Get<double>()); break;
	case EOnlineKeyValuePairDataType::Int64: OnlineStatValue.GetValue(OutStatValue.VariantData.Get<int64>()); break;
	default: 
		ensureMsgf(false, TEXT("Can't convert online stat value!"));
		break;
	}
}

FOnlineError FStatsOSSAdapter::ConvertUpdateUsersStatsV2toV1(const TArray<FUserStats>& UpdateUsersStats, TArray<FOnlineStatsUserUpdatedStats>& OutUpdateUserStatsV1)
{
	for (const FUserStats& UserStats : UpdateUsersStats)
	{
		const FUniqueNetIdPtr UserId = Auth->GetUniqueNetId(UserStats.AccountId);
		if (!UserId)
		{
			return Errors::InvalidUser();
		}

		FOnlineStatsUserUpdatedStats OnlineStatsUserUpdatedStats(UserId.ToSharedRef());
		for (const TPair<FString, FStatValue>& StatPair : UserStats.Stats)
		{
			FOnlineStatUpdate OnlineStatUpdate;
			ConvertStatValueV2ToStatUpdateV1(StatPair.Key, StatPair.Value, OnlineStatUpdate);
			OnlineStatsUserUpdatedStats.Stats.Emplace(StatPair.Key, OnlineStatUpdate);
		}

		OutUpdateUserStatsV1.Emplace(MoveTemp(OnlineStatsUserUpdatedStats));
	}

	return Errors::Success();
}

TOnlineAsyncOpHandle<FUpdateStats> FStatsOSSAdapter::UpdateStats(FUpdateStats::Params&& Params)
{
	TOnlineAsyncOpRef<FUpdateStats> Op = GetOp<FUpdateStats>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FUpdateStats>& Op)
	{
		const FUniqueNetIdPtr LocalUserId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return;
		}

		TArray<FOnlineStatsUserUpdatedStats> UpdateUserStatsV1;
		FOnlineError OnlineErrorConvert = ConvertUpdateUsersStatsV2toV1(Op.GetParams().UpdateUsersStats, UpdateUserStatsV1);
		if (OnlineErrorConvert != Errors::Success())
		{
			Op.SetError(MoveTemp(OnlineErrorConvert));
			return;
		}

		StatsInterface->UpdateStats(LocalUserId.ToSharedRef(), UpdateUserStatsV1, *MakeDelegateAdapter(this, [this, WeakOp = Op.AsWeak()](const FOnlineErrorOss& OnlineError) mutable
		{
			if (TSharedPtr<TOnlineAsyncOp<FUpdateStats>> PinnedOp = WeakOp.Pin())
			{
				if (!OnlineError.WasSuccessful())
				{
					PinnedOp->SetError(Errors::FromOssError(OnlineError));
					return;
				}

				PinnedOp->SetResult({});

				OnStatsUpdatedEvent.Broadcast(PinnedOp->GetParams());
			}
		}));
	}) 
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FQueryStats> FStatsOSSAdapter::QueryStats(FQueryStats::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryStats> Op = GetOp<FQueryStats>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FQueryStats>& Op)
	{
		const FUniqueNetIdPtr LocalUniqueNetId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);
		const FUniqueNetIdPtr TargetUniqueNetId = Auth->GetUniqueNetId(Op.GetParams().TargetAccountId);

		if (!LocalUniqueNetId || !TargetUniqueNetId)
		{
			Op.SetError(Errors::InvalidUser());
			return;
		}

		StatsInterface->QueryStats(LocalUniqueNetId.ToSharedRef(), TargetUniqueNetId.ToSharedRef(), *MakeDelegateAdapter(this, [this, WeakOp = Op.AsWeak()](const FOnlineErrorOss& OnlineError, const TSharedPtr<const FOnlineStatsUserStats>& QueriedStats) mutable
		{
			if (TSharedPtr<TOnlineAsyncOp<FQueryStats>> PinnedOp = WeakOp.Pin())
			{
				if (!OnlineError.WasSuccessful())
				{
					PinnedOp->SetError(Errors::FromOssError(OnlineError));
					return;
				}

				FQueryStats::Result Result;
				for (const TPair<FString, FOnlineStatValue>& StatPair : QueriedStats->Stats)
				{
					FStatValue StatValue;
					ConvertStatValueV1ToStatValueV2(StatPair.Value, StatValue);
					Result.Stats.Emplace(StatPair.Key, MoveTemp(StatValue));
				}

				FUserStats UserStats;
				UserStats.AccountId = PinnedOp->GetParams().TargetAccountId;
				UserStats.Stats = Result.Stats;
				CacheUserStats(UserStats);

				PinnedOp->SetResult(MoveTemp(Result));
			}
		}));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FBatchQueryStats> FStatsOSSAdapter::BatchQueryStats(FBatchQueryStats::Params&& Params)
{
	TOnlineAsyncOpRef<FBatchQueryStats> Op = GetOp<FBatchQueryStats>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FBatchQueryStats>& Op)
	{
		const FUniqueNetIdPtr LocalUserId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return;
		}

		TArray<FUniqueNetIdRef> NetIds;
		for (const FAccountId& TargetAccountId : Op.GetParams().TargetAccountIds)
		{
			const FUniqueNetIdPtr UniqueNetId = Auth->GetUniqueNetId(TargetAccountId);
			if (!UniqueNetId)
			{
				Op.SetError(Errors::InvalidUser());
				return;
			}

			NetIds.Add(UniqueNetId.ToSharedRef());
		}

		StatsInterface->QueryStats(LocalUserId.ToSharedRef(), NetIds, Op.GetParams().StatNames, *MakeDelegateAdapter(this, [this, WeakOp = Op.AsWeak()](const FOnlineErrorOss& OnlineError, const TArray<TSharedRef<const FOnlineStatsUserStats>>& UsersStatsResult) mutable
		{
			if (TSharedPtr<TOnlineAsyncOp<FBatchQueryStats>> PinnedOp = WeakOp.Pin())
			{
				if (!OnlineError.WasSuccessful())
				{
					PinnedOp->SetError(Errors::FromOssError(OnlineError));
				}

				FBatchQueryStats::Result Result;

				for (const TSharedRef<const FOnlineStatsUserStats>& OnlineStatsUserStats : UsersStatsResult)
				{
					FUserStats& UserStats = Result.UsersStats.Emplace_GetRef();
					UserStats.AccountId = Auth->GetAccountId(OnlineStatsUserStats->Account);

					for (const TPair<FString, FOnlineStatValue>& StatPair : OnlineStatsUserStats->Stats)
					{
						FStatValue StatValue;
						ConvertStatValueV1ToStatValueV2(StatPair.Value, StatValue);
						UserStats.Stats.Emplace(StatPair.Key, MoveTemp(StatValue));
					}

					CacheUserStats(UserStats);
				}

				PinnedOp->SetResult(MoveTemp(Result));
			}
		}));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

#if !UE_BUILD_SHIPPING
TOnlineAsyncOpHandle<FResetStats> FStatsOSSAdapter::ResetStats(FResetStats::Params&& Params)
{
	TOnlineAsyncOpRef<FResetStats> Op = GetOp<FResetStats>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FResetStats>& Op)
	{
		const FUniqueNetIdPtr LocalUserId = Auth->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return;
		}

		StatsInterface->ResetStats(LocalUserId.ToSharedRef());
		Op.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}
#endif // !UE_BUILD_SHIPPING

/* UE::Online */ }
