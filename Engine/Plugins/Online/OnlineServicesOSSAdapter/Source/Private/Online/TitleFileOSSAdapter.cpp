// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/TitleFileOSSAdapter.h"

#include "Interfaces/OnlineTitleFileInterface.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/DelegateAdapter.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/OnlineServicesOSSAdapter.h"

#include "OnlineSubsystem.h"

namespace UE::Online {

void FTitleFileOSSAdapter::PostInitialize()
{
	Super::PostInitialize();

	IOnlineSubsystem& SubsystemV1 = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem();

	TitleFileInterface = SubsystemV1.GetTitleFileInterface();
	check(TitleFileInterface);
}

TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> FTitleFileOSSAdapter::EnumerateFiles(FTitleFileEnumerateFiles::Params&& InParams)
{
	TOnlineAsyncOpRef<FTitleFileEnumerateFiles> Op = GetOp<FTitleFileEnumerateFiles>(MoveTemp(InParams));

	Op->Then([this](TOnlineAsyncOp<FTitleFileEnumerateFiles>& Op)
	{
		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, TitleFileInterface->OnEnumerateFilesCompleteDelegates, 
		[this, WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise)](bool bSuccess, const FString& ErrorStr) mutable
		{
			if (TOnlineAsyncOpPtr<FTitleFileEnumerateFiles> Op = WeakOp.Pin())
			{
				if (!bSuccess)
				{
					Op->SetError(Errors::Unknown());
				}
			}
			Promise.SetValue();
		});

		if (!TitleFileInterface->EnumerateFiles())
		{
			Op.SetError(Errors::Unknown());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		return Future;
	})
	.Then([this](TOnlineAsyncOp<FTitleFileEnumerateFiles>& InAsyncOp)
	{
		TArray<FCloudFileHeader> FileList;
		TitleFileInterface->GetFileList(FileList);

		TArray<FString> NewEnumeratedFiles;
		NewEnumeratedFiles.Reserve(FileList.Num());

		for (const FCloudFileHeader& File : FileList)
		{
			NewEnumeratedFiles.Emplace(File.FileName);
		}

		EnumeratedFiles.Emplace(MoveTemp(NewEnumeratedFiles));
		InAsyncOp.SetResult({});
	})
	.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

TOnlineResult<FTitleFileGetEnumeratedFiles> FTitleFileOSSAdapter::GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params)
{
	if (!EnumeratedFiles.IsSet())
	{
		// Call EnumerateFiles first
		return TOnlineResult<FTitleFileGetEnumeratedFiles>(Errors::InvalidState());
	}

	return TOnlineResult<FTitleFileGetEnumeratedFiles>({ *EnumeratedFiles });
}

TOnlineAsyncOpHandle<FTitleFileReadFile> FTitleFileOSSAdapter::ReadFile(FTitleFileReadFile::Params&& InParams)
{
	static const TCHAR* FileContentsKey = TEXT("FileContents");

	TOnlineAsyncOpRef<FTitleFileReadFile> Op = GetOp<FTitleFileReadFile>(MoveTemp(InParams));
	const FTitleFileReadFile::Params& Params = Op->GetParams();

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FTitleFileReadFile>& Op)
	{
		const FTitleFileReadFile::Params& Params = Op.GetParams();

		// First just try and read the file contents, if this succeeds we can return immediately.
		TArray<uint8> FileContents;
		if (TitleFileInterface->GetFileContents(Params.Filename, FileContents))
		{
			FTitleFileContentsRef FileContentsRef = MakeShared<FTitleFileContents>(MoveTemp(FileContents));
			Op.SetResult({FileContentsRef});
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, TitleFileInterface->OnEnumerateFilesCompleteDelegates,
			[this, WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise)](bool bSuccess, const FString& ErrorStr) mutable
		{
			if (TOnlineAsyncOpPtr<FTitleFileReadFile> Op = WeakOp.Pin())
			{
				if (!bSuccess)
				{
					Op->SetError(Errors::Unknown());
				}
			}
			Promise.SetValue();
		});

		if (!TitleFileInterface->ReadFile(Params.Filename))
		{
			Op.SetError(Errors::Unknown());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		return Future;
	})
	.Then([this](TOnlineAsyncOp<FTitleFileReadFile>& Op)
	{
		const FTitleFileReadFile::Params& Params = Op.GetParams();

		TArray<uint8> FileContents;
		if (TitleFileInterface->GetFileContents(Params.Filename, FileContents))
		{
			FTitleFileContentsRef FileContentsRef = MakeShared<FTitleFileContents>(MoveTemp(FileContents));
			Op.SetResult({ FileContentsRef });
		}
		else
		{
			Op.SetError(Errors::Unknown());
		}
	})
	.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

/* UE::Online */ }
