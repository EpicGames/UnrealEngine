// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/UserFileOSSAdapter.h"

#include "Interfaces/OnlineUserCloudInterface.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/DelegateAdapter.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/OnlineServicesOSSAdapter.h"

#include "OnlineSubsystem.h"

namespace UE::Online {

void FUserFileOSSAdapter::PostInitialize()
{
	Super::PostInitialize();

	IOnlineSubsystem& SubsystemV1 = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem();

	UserCloudInterface = SubsystemV1.GetUserCloudInterface();
	check(UserCloudInterface);
}

void FUserFileOSSAdapter::UpdateConfig()
{
	Super::UpdateConfig();
	LoadConfig(Config);
}

TOnlineAsyncOpHandle<FUserFileEnumerateFiles> FUserFileOSSAdapter::EnumerateFiles(FUserFileEnumerateFiles::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileEnumerateFiles> Op = GetOp<FUserFileEnumerateFiles>(MoveTemp(InParams));

	Op->Then([this](TOnlineAsyncOp<FUserFileEnumerateFiles>& Op)
	{
		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, UserCloudInterface->OnEnumerateUserFilesCompleteDelegates, 
		[WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise), LocalUserId](bool bSuccess, const FUniqueNetId& UserId) mutable
		{
			const bool bThisUser = *LocalUserId == UserId;
			if(bThisUser)
			{
				if (TOnlineAsyncOpPtr<FUserFileEnumerateFiles> Op = WeakOp.Pin())
				{
					if (!bSuccess)
					{
						Op->SetError(Errors::Unknown());
					}
				}
				Promise.SetValue();
			}
			return bThisUser;
		});

		UserCloudInterface->EnumerateUserFiles(*LocalUserId);

		return Future;
	})
	.Then([this](TOnlineAsyncOp<FUserFileEnumerateFiles>& Op)
	{
		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Op.GetParams().LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return;
		}

		TArray<FCloudFileHeader> FileList;
		UserCloudInterface->GetUserFileList(*LocalUserId, FileList);

		TArray<FString> NewEnumeratedFiles;
		NewEnumeratedFiles.Reserve(FileList.Num());

		for (const FCloudFileHeader& File : FileList)
		{
			NewEnumeratedFiles.Emplace(File.FileName);
		}

		EnumeratedFiles.Emplace(Op.GetParams().LocalAccountId, MoveTemp(NewEnumeratedFiles));
		Op.SetResult({});
	})
	.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

TOnlineResult<FUserFileGetEnumeratedFiles> FUserFileOSSAdapter::GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params)
{
	const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
	if (!LocalUserId)
	{
		return TOnlineResult<FUserFileGetEnumeratedFiles>(Errors::InvalidUser());
	}

	const TArray<FString>* UserFiles = EnumeratedFiles.Find(Params.LocalAccountId);
	if (!UserFiles)
	{
		// Call EnumerateFiles first
		return TOnlineResult<FUserFileGetEnumeratedFiles>(Errors::InvalidState());
	}

	return TOnlineResult<FUserFileGetEnumeratedFiles>({ *UserFiles });
}

TOnlineAsyncOpHandle<FUserFileReadFile> FUserFileOSSAdapter::ReadFile(FUserFileReadFile::Params&& InParams)
{
	static const TCHAR* FileContentsKey = TEXT("FileContents");

	TOnlineAsyncOpRef<FUserFileReadFile> Op = GetOp<FUserFileReadFile>(MoveTemp(InParams));
	const FUserFileReadFile::Params& Params = Op->GetParams();

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FUserFileReadFile>& Op)
	{
		const FUserFileReadFile::Params& Params = Op.GetParams();

		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		// First just try and read the file contents, if this succeeds we can return immediately.
		TArray<uint8> FileContents;
		if (UserCloudInterface->GetFileContents(*LocalUserId, Params.Filename, FileContents))
		{
			FUserFileContentsRef FileContentsRef = MakeShared<FUserFileContents>(MoveTemp(FileContents));
			Op.SetResult({FileContentsRef});
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, UserCloudInterface->OnReadUserFileCompleteDelegates,
			[WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise), LocalUserId, Filename = Params.Filename](bool bSuccess, const FUniqueNetId& UserId, const FString& InFilename) mutable
		{
			const bool bThisUser = *LocalUserId == UserId;
			const bool bThisFile = Filename == InFilename;
			const bool bThis = bThisUser && bThisFile;
			if (bThis)
			{
				if (TOnlineAsyncOpPtr<FUserFileReadFile> Op = WeakOp.Pin())
				{
					if (!bSuccess)
					{
						Op->SetError(Errors::Unknown());
					}
				}
				Promise.SetValue();
			}
			return bThis;
		});

		if (!UserCloudInterface->ReadUserFile(*LocalUserId, Params.Filename))
		{
			Op.SetError(Errors::Unknown());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		return Future;
	})
	.Then([this](TOnlineAsyncOp<FUserFileReadFile>& Op)
	{
		const FUserFileReadFile::Params& Params = Op.GetParams();

		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return;
		}

		TArray<uint8> FileContents;
		if (UserCloudInterface->GetFileContents(*LocalUserId, Params.Filename, FileContents))
		{
			FUserFileContentsRef FileContentsRef = MakeShared<FUserFileContents>(MoveTemp(FileContents));
			Op.SetResult({ FileContentsRef });
		}
		else
		{
			Op.SetError(Errors::Unknown());
		}
	})
	.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileWriteFile> FUserFileOSSAdapter::WriteFile(FUserFileWriteFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileWriteFile> Op = GetOp<FUserFileWriteFile>(MoveTemp(InParams));
	const FUserFileWriteFile::Params& Params = Op->GetParams();

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FUserFileWriteFile>& Op)
	{
		const FUserFileWriteFile::Params& Params = Op.GetParams();

		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, UserCloudInterface->OnWriteUserFileCompleteDelegates,
			[WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise), LocalUserId, Filename = Params.Filename](bool bSuccess, const FUniqueNetId& UserId, const FString& InFilename) mutable
		{
			const bool bThisUser = *LocalUserId == UserId;
			const bool bThisFile = Filename == InFilename;
			const bool bThis = bThisUser && bThisFile;
			if (bThis)
			{
				if (TOnlineAsyncOpPtr<FUserFileWriteFile> Op = WeakOp.Pin())
				{
					if (bSuccess)
					{
						Op->SetResult({});
					}
					else
					{
						Op->SetError(Errors::Unknown());
					}
				}
				Promise.SetValue();
			}
			return bThis;
		});

		// WriteUserFile takes a non-const ref, not sure why.
		FUserFileContents& FileContents = const_cast<FUserFileContents&>(Params.FileContents);

		if (!UserCloudInterface->WriteUserFile(*LocalUserId, Params.Filename, FileContents, Config.bCompressBeforeUpload))
		{
			Op.SetError(Errors::Unknown());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		return Future;
	})
	.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileDeleteFile> FUserFileOSSAdapter::DeleteFile(FUserFileDeleteFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileDeleteFile> Op = GetOp<FUserFileDeleteFile>(MoveTemp(InParams));
	const FUserFileDeleteFile::Params& Params = Op->GetParams();

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FUserFileDeleteFile>& Op)
	{
		const FUserFileDeleteFile::Params& Params = Op.GetParams();

		const FUniqueNetIdPtr LocalUserId = Services.Get<FAuthOSSAdapter>()->GetUniqueNetId(Params.LocalAccountId);
		if (!LocalUserId)
		{
			Op.SetError(Errors::InvalidUser());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, UserCloudInterface->OnDeleteUserFileCompleteDelegates,
			[WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise), LocalUserId, Filename = Params.Filename](bool bSuccess, const FUniqueNetId& UserId, const FString& InFilename) mutable
		{
			const bool bThisUser = *LocalUserId == UserId;
			const bool bThisFile = Filename == InFilename;
			const bool bThis = bThisUser && bThisFile;
			if (bThis)
			{
				if (TOnlineAsyncOpPtr<FUserFileDeleteFile> Op = WeakOp.Pin())
				{
					if (bSuccess)
					{
						Op->SetResult({});
					}
					else
					{
						Op->SetError(Errors::Unknown());
					}
				}
				Promise.SetValue();
			}
			return bThis;
		});

		const bool bShouldCloudDelete = true;
		const bool bShouldLocallyDelete = true;
		if (!UserCloudInterface->DeleteUserFile(*LocalUserId, Params.Filename, bShouldCloudDelete, bShouldLocallyDelete))
		{
			Op.SetError(Errors::Unknown());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		return Future;
	})
	.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

/* UE::Online */ }
