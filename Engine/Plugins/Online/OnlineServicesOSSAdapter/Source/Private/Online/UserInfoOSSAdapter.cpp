// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/UserInfoOSSAdapter.h"

#include "Interfaces/OnlineUserInterface.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/DelegateAdapter.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineIdOSSAdapter.h"
#include "Online/OnlineServicesOSSAdapter.h"

#include "OnlineSubsystem.h"

namespace UE::Online {

void FUserInfoOSSAdapter::Initialize()
{
	Super::Initialize();

	IOnlineSubsystem& Subsystem = static_cast<FOnlineServicesOSSAdapter&>(Services).GetSubsystem();

	UserInterface = Subsystem.GetUserInterface();
	check(UserInterface);
}

TOnlineAsyncOpHandle<FQueryUserInfo> FUserInfoOSSAdapter::QueryUserInfo(FQueryUserInfo::Params&& InParams)
{
	TOnlineAsyncOpRef<FQueryUserInfo> Op = GetJoinableOp<FQueryUserInfo>(MoveTemp(InParams));
	const FQueryUserInfo::Params& Params = Op->GetParams();

	if (Params.AccountIds.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FQueryUserInfo>& Op)
	{
		const FQueryUserInfo::Params& Params = Op.GetParams();
		const FAuthOSSAdapter* AuthInterface = Services.Get<FAuthOSSAdapter>();

		const int32 LocalUserNum = AuthInterface->GetLocalUserNum(Params.LocalAccountId);
		if (LocalUserNum == INDEX_NONE)
		{
			Op.SetError(Errors::InvalidUser());
			return MakeFulfilledPromise<void>().GetFuture();
		}

		TArray<FUniqueNetIdRef> UserIds;
		UserIds.Reserve(Params.AccountIds.Num());
		for(const FAccountId& TargetAccountId : Params.AccountIds)
		{
			FUniqueNetIdPtr TargetUserNetId = AuthInterface->GetUniqueNetId(TargetAccountId);
			if (!TargetUserNetId)
			{
				Op.SetError(Errors::InvalidParams());
				return MakeFulfilledPromise<void>().GetFuture();
			}
			UserIds.Emplace(TargetUserNetId.ToSharedRef());
		}

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		MakeMulticastAdapter(this, UserInterface->OnQueryUserInfoCompleteDelegates[LocalUserNum],
			[WeakOp = Op.AsWeak(), Promise = MoveTemp(Promise), UserIds](int32 LocalUserNum, bool bSuccess, const TArray<FUniqueNetIdRef>& InUserIds, const FString& ErrorStr) mutable
		{
			const bool bShouldHandle = InUserIds == UserIds;
			if (bShouldHandle)
			{
				if (TOnlineAsyncOpPtr<FQueryUserInfo> Op = WeakOp.Pin())
				{
					if (bSuccess)
					{
						Op->SetResult({});
					}
					else
					{
						Op->SetError(Errors::Unknown());
					}
				}
				Promise.SetValue();
			}
			return bShouldHandle;
		});

		UserInterface->QueryUserInfo(LocalUserNum, UserIds);

		return Future;
	});

	Op->Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

TOnlineResult<FGetUserInfo> FUserInfoOSSAdapter::GetUserInfo(FGetUserInfo::Params&& Params)
{
	const FAuthOSSAdapter* AuthInterface = Services.Get<FAuthOSSAdapter>();

	const int32 LocalUserNum = AuthInterface->GetLocalUserNum(Params.LocalAccountId);
	if (LocalUserNum == INDEX_NONE)
	{
		return TOnlineResult<FGetUserInfo>(Errors::InvalidUser());
	}

	const FUniqueNetIdPtr TargetUserNetId = AuthInterface->GetUniqueNetId(Params.AccountId);
	if (!TargetUserNetId)
	{
		return TOnlineResult<FGetUserInfo>(Errors::InvalidParams());
	}

	const TSharedPtr<FOnlineUser> OnlineUser = UserInterface->GetUserInfo(LocalUserNum, *TargetUserNetId);
	if (!OnlineUser)
	{
		return TOnlineResult<FGetUserInfo>(Errors::NotFound());
	}

	TSharedRef<FUserInfo> UserInfo = MakeShared<FUserInfo>();
	UserInfo->AccountId = Params.AccountId;
	UserInfo->DisplayName = OnlineUser->GetDisplayName();

	return TOnlineResult<FGetUserInfo>({ UserInfo });
}

/* UE::Online */ }
