// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/AchievementsCommon.h"

#include "OnlineSubsystemTypes.h"

using IOnlineAchievementsPtr = TSharedPtr<class IOnlineAchievements>;
using IOnlineExternalUIPtr = TSharedPtr<class IOnlineExternalUI>;

namespace UE::Online {

class FAchievementsOSSAdapter : public FAchievementsCommon
{
public:
	using Super = FAchievementsCommon;

	using FAchievementsCommon::FAchievementsCommon;

	// IOnlineComponent
	virtual void Initialize() override;
	virtual void PostInitialize() override;
	virtual void PreShutdown() override;

	// IAchievements
	virtual TOnlineAsyncOpHandle<FQueryAchievementDefinitions> QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params) override;
	virtual TOnlineResult<FGetAchievementIds> GetAchievementIds(FGetAchievementIds::Params&& Params) override;
	virtual TOnlineResult<FGetAchievementDefinition> GetAchievementDefinition(FGetAchievementDefinition::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FQueryAchievementStates> QueryAchievementStates(FQueryAchievementStates::Params&& Params) override;
	virtual TOnlineResult<FGetAchievementState> GetAchievementState(FGetAchievementState::Params&& Params) const override;
	// Intentionally unimplemented, all v1 achis are stats based
	//virtual TOnlineAsyncOpHandle<FUnlockAchievements> UnlockAchievements(FUnlockAchievements::Params&& Params) override;
	virtual TOnlineResult<FDisplayAchievementUI> DisplayAchievementUI(FDisplayAchievementUI::Params&& Params) override;

protected:
	IOnlineAchievementsPtr AchievementsInterface = nullptr;
	IOnlineExternalUIPtr ExternalUIInterface = nullptr;

	using FAchievementDefinitionMap = TMap<FString, FAchievementDefinition>;
	TOptional<FAchievementDefinitionMap> AchievementDefinitions;
};

/* UE::Online */ }
