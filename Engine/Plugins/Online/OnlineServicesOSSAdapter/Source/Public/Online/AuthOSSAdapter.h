// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/AuthCommon.h"

#include "OnlineSubsystemTypes.h"

#define UE_API ONLINESERVICESOSSADAPTER_API

class IOnlineSubsystem;
class IOnlineIdentity;
using IOnlineIdentityPtr = TSharedPtr<IOnlineIdentity>;

namespace UE::Online {

class FOnlineServicesOSSAdapter;

struct FAuthHandleLoginStatusChangedImpl
{
	static constexpr TCHAR Name[] = TEXT("HandleLoginStatusChangedImpl");

	struct Params
	{
		FPlatformUserId PlatformUserId;
		FAccountId AccountId;
		ELoginStatus NewLoginStatus;
	};

	struct Result
	{
	};
};

struct FAccountInfoOSSAdapter final : public FAccountInfo
{
	FUniqueNetIdPtr UniqueNetId;
	int32 LocalUserNum = INDEX_NONE;
};

class FAccountInfoRegistryOSSAdapter final : public FAccountInfoRegistry
{
public:
	using Super = FAccountInfoRegistry;

	virtual ~FAccountInfoRegistryOSSAdapter() = default;

	ONLINESERVICESOSSADAPTER_API TSharedPtr<FAccountInfoOSSAdapter> Find(FPlatformUserId PlatformUserId) const;
	ONLINESERVICESOSSADAPTER_API TSharedPtr<FAccountInfoOSSAdapter> Find(FAccountId AccountId) const;

	void Register(const TSharedRef<FAccountInfoOSSAdapter>&UserAuthData);
	void Unregister(FAccountId AccountId);
};

using FAuthOSSAdapterPtr = TSharedPtr<class FAuthOSSAdapter>;

class FAuthOSSAdapter : public FAuthCommon
{
public:
	using Super = FAuthCommon;

	using FAuthCommon::FAuthCommon;

	// IOnlineComponent
	UE_API virtual void PostInitialize() override;
	UE_API virtual void PreShutdown() override;

	// IAuth
	UE_API virtual TOnlineAsyncOpHandle<FAuthLogin> Login(FAuthLogin::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthLogout> Logout(FAuthLogout::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthQueryExternalServerAuthTicket> QueryExternalServerAuthTicket(FAuthQueryExternalServerAuthTicket::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthQueryExternalAuthToken> QueryExternalAuthToken(FAuthQueryExternalAuthToken::Params&& Params) override;

	UE_API FUniqueNetIdPtr GetUniqueNetId(FAccountId AccountId) const;
	UE_API FAccountId GetAccountId(const FUniqueNetIdRef& UniqueNetId) const;
	UE_API int32 GetLocalUserNum(FAccountId AccountId) const;

protected:
#if !UE_BUILD_SHIPPING
	static UE_API void CheckMetadata();
#endif

	UE_API virtual const FAccountInfoRegistry& GetAccountInfoRegistry() const override;

	UE_API const FOnlineServicesOSSAdapter& GetOnlineServicesOSSAdapter() const;
	UE_API FOnlineServicesOSSAdapter& GetOnlineServicesOSSAdapter();
	UE_API const IOnlineSubsystem& GetSubsystem() const;
	UE_API IOnlineIdentityPtr GetIdentityInterface() const;

	UE_API TOnlineAsyncOpHandle<FAuthHandleLoginStatusChangedImpl> HandleLoginStatusChangedImplOp(FAuthHandleLoginStatusChangedImpl::Params&& Params);

	UE_API bool PopulateAttributes(FAccountInfoOSSAdapter& AccountInfoOSSAdapter) const;

	FAccountInfoRegistryOSSAdapter AccountInfoRegistryOSSAdapter;
	FDelegateHandle OnLoginStatusChangedHandle[MAX_LOCAL_PLAYERS];
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FAuthHandleLoginStatusChangedImpl::Params)
	ONLINE_STRUCT_FIELD(FAuthHandleLoginStatusChangedImpl::Params, PlatformUserId),
	ONLINE_STRUCT_FIELD(FAuthHandleLoginStatusChangedImpl::Params, AccountId),
	ONLINE_STRUCT_FIELD(FAuthHandleLoginStatusChangedImpl::Params, NewLoginStatus)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthHandleLoginStatusChangedImpl::Result)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }

#undef UE_API
