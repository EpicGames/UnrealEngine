// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CommerceCommon.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/OnlineComponent.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlinePurchaseInterface.h"
#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "Interfaces/OnlineExternalUIInterface.h"

namespace UE::Online {

class FOnlineServicesOSSAdapter;

class FCommerceOSSAdapter : public FCommerceCommon
{
public:
	using Super = FCommerceCommon;
	using FCommerceCommon::FCommerceCommon;

	// IOnlineComponent
	virtual void PostInitialize() override;
	virtual void PreShutdown() override;

	// ICommerce
	virtual TOnlineAsyncOpHandle<FCommerceQueryOffers> QueryOffers(FCommerceQueryOffers::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FCommerceQueryOffersById> QueryOffersById(FCommerceQueryOffersById::Params&& Params) override;
	virtual TOnlineResult<FCommerceGetOffers> GetOffers(FCommerceGetOffers::Params&& Params) override;
	virtual TOnlineResult<FCommerceGetOffersById> GetOffersById(FCommerceGetOffersById::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FCommerceShowStoreUI> ShowStoreUI(FCommerceShowStoreUI::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FCommerceCheckout> Checkout(FCommerceCheckout::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FCommerceQueryTransactionEntitlements> QueryTransactionEntitlements(FCommerceQueryTransactionEntitlements::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FCommerceQueryEntitlements> QueryEntitlements(FCommerceQueryEntitlements::Params&& Params) override;
	virtual TOnlineResult<FCommerceGetEntitlements> GetEntitlements(FCommerceGetEntitlements::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FCommerceRedeemEntitlement> RedeemEntitlement(FCommerceRedeemEntitlement::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FCommerceRetrieveS2SToken> RetrieveS2SToken(FCommerceRetrieveS2SToken::Params&& Params) override;

protected:

	const FOnlineServicesOSSAdapter& GetOnlineServicesOSSAdapter() const;
	FOnlineServicesOSSAdapter& GetOnlineServicesOSSAdapter();
	const IOnlineSubsystem& GetSubsystem() const;
	IOnlinePurchasePtr GetPurchaseInterface() const;
	IOnlineStoreV2Ptr GetStoreInterface() const;
	IOnlineExternalUIPtr GetExternalUIInterface() const;

	FAuthOSSAdapter* Auth;
};

/* UE::Online */
}
