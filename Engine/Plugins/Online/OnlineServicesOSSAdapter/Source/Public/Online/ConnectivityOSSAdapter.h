// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/ConnectivityCommon.h"

namespace UE::Online {

class FConnectivityOSSAdapter : public FConnectivityCommon
{
public:
	using Super = FConnectivityCommon;

	using FConnectivityCommon::FConnectivityCommon;

	// IOnlineComponent
	virtual void PostInitialize() override;
	virtual void PreShutdown() override;

	// IConnectivity
	virtual TOnlineResult<FGetConnectionStatus> GetConnectionStatus(FGetConnectionStatus::Params&& Params) override;

protected:
	FDelegateHandle OnConnectionStatusChangedHandle;
	TMap<FString, EOnlineServicesConnectionStatus> CurrentStatus;
};

/* UE::Online */ }
