// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "OnlineSubsystemTypes.h"

// this isn't actually used, but it allows files to only include this instead of both for ease-of-use
#include "Online/MulticastAdapter.h"

/*
	BASIC USAGE
		The Delegate Adapter can be used in place of an FOn{Name}Delegate::CreateLambda call:
			
			Identity->GetAuthToken(LocalUserNum, FOnAuthTokenGetComplete::CreateLambda[this](int32 UserNum, FString LoginToken){ ... });
			->
			Identity->GetAuthToken(LocalUserNum, *MakeDelegateAdapter(this, [this](int32 UserNum, FString LoginToken){ ... });

		The second instance, in addition to being slightly shorter and easier to remember, will be a unique ptr (can MoveTemp into the closure) and also will automatically do a weak ptr validity check on "this"

		The Delegate Adapter can also be used to bind to TDelegates:

			TDelegate<void(bool)> Delegate;
			Delegate = *MakeDelegateAdapter(this, [this](bool bParam) {});

		This is useful when you need to capture something non-copyable, e.g. a TPromise, as TDelegate's need to be copyable.

	The lifetime of the adapter is a shared ptr attached to the input delegate and will live as long as that delegate is bound.
*/

namespace UE::Online {

template<typename ComponentType, typename... LambdaArgs>
class TDelegateAdapter
	: public TSharedFromThis<TDelegateAdapter<ComponentType, LambdaArgs...>>
{

public:
	TDelegateAdapter(TSharedRef<ComponentType>& InParent)
		: Parent(TWeakPtr<ComponentType>(InParent))
	{
	}
	
	void SetupDelegate(TUniqueFunction<void(LambdaArgs...)>&& InCallback)
	{
		// callback exists outside of the delegate's bound lambda since that lambda is not move-only
		Callback = [this, InCallback = MoveTemp(InCallback)](LambdaArgs... Args)
		{
			InCallback(Forward<LambdaArgs>(Args)...);
		};

	}

	operator TDelegate<void(LambdaArgs...)>()
	{
		TSharedPtr<TDelegateAdapter<ComponentType, LambdaArgs...>> SharedAnchor = this->AsShared();
		return TDelegate<void(LambdaArgs...)>::CreateLambda([this, SharedAnchor](LambdaArgs... Args)
		{
			if(Parent.IsValid())
			{
				Callback(Forward<LambdaArgs>(Args)...);
			}
		});
	}

	operator TTSDelegate<void(LambdaArgs...)>()
	{
		TSharedPtr<TDelegateAdapter<ComponentType, LambdaArgs...>> SharedAnchor = this->AsShared();
		return TTSDelegate<void(LambdaArgs...)>::CreateLambda([this, SharedAnchor](LambdaArgs... Args)
		{
			if(Parent.IsValid())
			{
				Callback(Forward<LambdaArgs>(Args)...);
			}
		});
	}

private:
	TWeakPtr<ComponentType> Parent;
	TUniqueFunction<void(LambdaArgs...)> Callback;
	FDelegateHandle Handle;
};

/*
	These set of classes allow us to accept a generic lambda and decompose its parameters into a TUniqueFunction so we have the exact list of parameters to bind/unbind the delegate to
*/
namespace Private { 
	template <typename... Params>
	class TDAConverterHelper2
	{
	};

	template <typename CallableObject, typename TResultType, typename... ParamTypes>
	class TDAConverterHelper2<CallableObject, TResultType, ParamTypes...>
	{
	public:

		TUniqueFunction<TResultType(ParamTypes...)> GetUniqueFn(CallableObject&& Callable)
		{
			return TUniqueFunction<TResultType(ParamTypes...)>(MoveTemp(Callable));
		}

		template<typename ComponentType>
		TSharedPtr<TDelegateAdapter<ComponentType, ParamTypes...>> Construct(TSharedRef<ComponentType> Interface, CallableObject&& InCallback)
		{
 			TSharedPtr<TDelegateAdapter<ComponentType, ParamTypes...>> Adapter = MakeShared<TDelegateAdapter<ComponentType, ParamTypes...>>(Interface);
			Adapter->SetupDelegate(GetUniqueFn(MoveTemp(InCallback)));
			return Adapter;
		}
	};

	template <typename CallableObject, typename = void>
	class TDAConverterHelper
	{
	};

	template <typename CallableObject, typename ReturnType, typename... ParamTypes>
	class TDAConverterHelper<CallableObject, ReturnType(ParamTypes...)>
		: public TDAConverterHelper2<CallableObject, ReturnType, ParamTypes...>
	{
	};


	template <typename CallableObject, typename ReturnType, typename ObjectType, typename... ParamTypes>
	class TDAConverterHelper<CallableObject, ReturnType(ObjectType::*)(ParamTypes...)>
		: public TDAConverterHelper2<CallableObject, ReturnType, ParamTypes...>
	{
	};

	template <typename CallableObject, typename ReturnType, typename ObjectType, typename... ParamTypes>
	class TDAConverterHelper<CallableObject, ReturnType(ObjectType::*)(ParamTypes...) const>
		: public TDAConverterHelper2<CallableObject, ReturnType, ParamTypes...>
	{
	};

	template <typename CallableObject>
	class TDAConverter
		: public TDAConverterHelper<CallableObject, decltype(&std::remove_reference_t<CallableObject>::operator())>
	{
	};
} // namespace Private (UE::Online)


template<typename ComponentType, typename Callback>
auto MakeDelegateAdapter(TSharedRef<ComponentType> Interface, Callback&& InCallback)
{
	return Private::TDAConverter<Callback>().Construct(Interface, MoveTemp(InCallback));
}

template<typename ComponentType, typename Callback>
auto MakeDelegateAdapter(TSharedPtr<ComponentType> Interface, Callback&& InCallback)
{
	checkf(Interface.IsValid(), TEXT("Must pass in a valid interface to MakeDelegateAdapter!"));
	return Private::TDAConverter<Callback>().Construct(Interface.ToSharedRef(), MoveTemp(InCallback));
}

template<typename ComponentType, typename Callback>
auto MakeDelegateAdapter(ComponentType* Interface, Callback&& InCallback)
{
	return Private::TDAConverter<Callback>().Construct(Interface->AsShared(), MoveTemp(InCallback));
}

template<typename ComponentType, typename Callback>
auto MakeDelegateAdapter(ComponentType& Interface, Callback&& InCallback)
{
	return Private::TDAConverter<Callback>().Construct(Interface.AsShared(), MoveTemp(InCallback));
}

/* UE::Online */ }
