// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Online/OnlineError.h"

struct FOnlineError;
using FOnlineErrorOss = ::FOnlineError;

namespace UE::Online::Errors {

	UE_ONLINE_ERROR_CATEGORY(Oss, Engine, 0x7, "Oss")
	ONLINESERVICESOSSADAPTER_API FOnlineError FromOssError(const FOnlineErrorOss& Result);
	ONLINESERVICESOSSADAPTER_API FOnlineError FromOssErrorCode(const FString& ErrorCode);
	
	// if you would like to add a parser that transforms custom platform errors into readable online service types, you can call this in your platform's error handler
	using TOssPlatformErrorHandler = TUniqueFunction<TOptional<FOnlineError>(const FOnlineError&, const FOnlineErrorOss&)>;
	void AddOssPlatformErrorHandler(TOssPlatformErrorHandler&& InFunction);


} // namespace UE::Online::Errors


ONLINESERVICESOSSADAPTER_API bool operator==(const UE::Online::FOnlineError& Left, const FOnlineErrorOss& Right);
ONLINESERVICESOSSADAPTER_API bool operator==(const FOnlineErrorOss& Left, const UE::Online::FOnlineError& Right);
ONLINESERVICESOSSADAPTER_API bool operator!=(const UE::Online::FOnlineError& Left, const FOnlineErrorOss& Right);
ONLINESERVICESOSSADAPTER_API bool operator!=(const FOnlineErrorOss& Left, const UE::Online::FOnlineError& Right);