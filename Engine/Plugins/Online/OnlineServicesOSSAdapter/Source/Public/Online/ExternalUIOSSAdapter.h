// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/ExternalUICommon.h"

class IOnlineExternalUI;
using IOnlineExternalUIPtr = TSharedPtr<IOnlineExternalUI>;

namespace UE::Online {

class FExternalUIOSSAdapter : public FExternalUICommon
{
public:
	using Super = FExternalUICommon;

	using FExternalUICommon::FExternalUICommon;

	// TOnlineComponent
	virtual void PostInitialize() override;
	virtual void PreShutdown() override;

	// IExternalUI
	virtual TOnlineAsyncOpHandle<FExternalUIShowLoginUI> ShowLoginUI(FExternalUIShowLoginUI::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FExternalUIShowFriendsUI> ShowFriendsUI(FExternalUIShowFriendsUI::Params&& Params) override;

	IOnlineExternalUIPtr GetExternalUIInterface() const;

protected:
	FDelegateHandle OnExternalUIChangeHandle;
};

/* UE::Online */ }
