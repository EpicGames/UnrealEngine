// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/LeaderboardsCommon.h"
#include "AuthOSSAdapter.h"
#include "OnlineSubsystemTypes.h"
#include "Interfaces/OnlineLeaderboardInterface.h"

class IOnlineLeaderboards;
using IOnlineLeaderboardsPtr = TSharedPtr<IOnlineLeaderboards>;

namespace UE::Online {

class FLeaderboardsOSSAdapter : public FLeaderboardsCommon
{
public:
	using Super = FLeaderboardsCommon;

	using FLeaderboardsCommon::FLeaderboardsCommon;

	// IOnlineComponent
	virtual void PostInitialize() override;

	// ILeaderboards
	virtual TOnlineAsyncOpHandle<FReadEntriesForUsers> ReadEntriesForUsers(FReadEntriesForUsers::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FReadEntriesAroundRank> ReadEntriesAroundRank(FReadEntriesAroundRank::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FReadEntriesAroundUser> ReadEntriesAroundUser(FReadEntriesAroundUser::Params&& Params) override;

protected:
	TOptional<FOnlineError> PrepareLeaderboardReadObject(const FString& BoardName);
	void ReadLeaderboardsResultFromV1ToV2(const FString& BoardName, TArray<FLeaderboardEntry>& OutEntries);

	const FAuthOSSAdapter* Auth;
	IOnlineLeaderboardsPtr LeaderboardsInterface;
	FOnlineLeaderboardReadPtr ReadObject;
};

/* UE::Online */ }
