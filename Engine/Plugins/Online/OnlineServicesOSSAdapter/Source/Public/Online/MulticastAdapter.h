// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "OnlineSubsystemTypes.h"

#include <type_traits>

/*
	BASIC USAGE
		The multicast adapter maintains the same functionality as a delegate adapter

			Identity->OnLoginCompleteDelegate.BindLambda([this](int32 UserNum){...})
			->
			MakeMulticastAdapter(this, Identity->OnLoginComplete, [this](int32 UserNum){...});

		When this adapter unbinds itself depends on the return type of the lambda bound to it:
		- void - the adapter will unbind itself after a single execution
		- bool - the adapter will unbind itself when the lambda returns true, and will stay bound when it returns false.

		The lifetime of the adapter is a shared ptr attached to the input delegate and will live as long as that delegate is bound.
*/
namespace UE::Online {

template<typename ComponentType, typename DelegateType, typename LambdaRet, typename... LambdaArgs>
class TMulticastDelegateAdapter
	: public TSharedFromThis<TMulticastDelegateAdapter<ComponentType, DelegateType, LambdaRet, LambdaArgs...>>
{

public:
	TMulticastDelegateAdapter(TSharedRef<ComponentType>& InParent, DelegateType& InDelegate)
		: Parent(TWeakPtr<ComponentType>(InParent))
		, Delegate(InDelegate)
	{
	}

	void SetupDelegate(TUniqueFunction<LambdaRet(LambdaArgs...)>&& InCallback)
	{
		TSharedPtr<TMulticastDelegateAdapter<ComponentType, DelegateType, LambdaRet, LambdaArgs...>> SharedAnchor = this->AsShared();

		// callback exists outside of the delegate's bound lambda since that lambda is not move-only
		Callback = [this, InCallback = MoveTemp(InCallback)](LambdaArgs... Args)
		{
			return InCallback(Forward<LambdaArgs>(Args)...);
		};

		Handle = Delegate.AddLambda([this, SharedAnchor](LambdaArgs... Args) mutable 
		{
			bool bUnbind = true;
			if(Parent.IsValid())
			{
				if constexpr (std::is_void_v<LambdaRet>)
				{
					Callback(Forward<LambdaArgs>(Args)...);
				}
				else
				{
					bUnbind = Callback(Forward<LambdaArgs>(Args)...);
				}
			}

			if (bUnbind)
			{
				// Calling remove will invalidate the closure data and destruct SharedAnchor.
				Delegate.Remove(Handle);
			}
		});
	}

	FDelegateHandle GetHandle()
	{
		return Handle;
	}

	TWeakPtr<TMulticastDelegateAdapter<ComponentType, DelegateType, LambdaRet, LambdaArgs...>> AsWeak()
	{
		return TWeakPtr<TMulticastDelegateAdapter<ComponentType, DelegateType, LambdaRet, LambdaArgs...>>(this->AsShared());
	}

private:
	TWeakPtr<ComponentType> Parent;
	TUniqueFunction<LambdaRet(LambdaArgs...)> Callback;
	DelegateType& Delegate;
	FDelegateHandle Handle;
};

/*
	These set of classes allow us to accept a generic lambda and decompose its parameters into a TUniqueFunction so we have the exact list of parameters to bind/unbind the delegate to
*/
namespace Private 
{
	template <typename... Params>
	class TMAConverterHelper2
	{
	};

	template <typename CallableObject, typename TResultType, typename... ParamTypes>
	class TMAConverterHelper2<CallableObject, TResultType, ParamTypes...>
	{
	public:

		TUniqueFunction<TResultType(ParamTypes...)> GetUniqueFn(CallableObject&& Callable)
		{
			return TUniqueFunction<TResultType(ParamTypes...)>(MoveTemp(Callable));
		}

		template<typename ComponentType, typename DelegateType>
		TSharedPtr<TMulticastDelegateAdapter<ComponentType, DelegateType, TResultType, ParamTypes...>> Construct(TSharedRef<ComponentType> Interface, DelegateType& InDelegate, CallableObject&& InCallback)
		{
			TSharedPtr<TMulticastDelegateAdapter<ComponentType, DelegateType, TResultType, ParamTypes...>> Adapter = MakeShared<TMulticastDelegateAdapter<ComponentType, DelegateType, TResultType, ParamTypes...>>(Interface, InDelegate);
			Adapter->SetupDelegate(GetUniqueFn(MoveTemp(InCallback)));
			return Adapter;
		}
	};

	template <typename CallableObject, typename = void>
	class TMAConverterHelper
	{
	};

	template <typename CallableObject, typename ReturnType, typename... ParamTypes>
	class TMAConverterHelper<CallableObject, ReturnType(ParamTypes...)>
		: public TMAConverterHelper2<CallableObject, ReturnType, ParamTypes...>
	{
	};


	template <typename CallableObject, typename ReturnType, typename ObjectType, typename... ParamTypes>
	class TMAConverterHelper<CallableObject, ReturnType(ObjectType::*)(ParamTypes...)>
		: public TMAConverterHelper2<CallableObject, ReturnType, ParamTypes...>
	{
	};

	template <typename CallableObject, typename ReturnType, typename ObjectType, typename... ParamTypes>
	class TMAConverterHelper<CallableObject, ReturnType(ObjectType::*)(ParamTypes...) const>
		: public TMAConverterHelper2<CallableObject, ReturnType, ParamTypes...>
	{
	};

	template <typename CallableObject>
	class TMAConverter
		: public TMAConverterHelper<CallableObject, decltype(&std::remove_reference_t<CallableObject>::operator())>
	{
	};
} // namespace Private (UE::Online)

template<typename ComponentType, typename DelegateType, typename Callback>
auto MakeMulticastAdapter(TSharedRef<ComponentType> Interface, DelegateType& InDelegate, Callback&& InCallback)
{
	return Private::TMAConverter<Callback>().Construct(Interface, InDelegate, MoveTemp(InCallback));
}

template<typename ComponentType, typename DelegateType, typename Callback>
auto MakeMulticastAdapter(TSharedPtr<ComponentType> Interface, DelegateType& InDelegate, Callback&& InCallback)
{
	checkf(Interface.IsValid(), TEXT("Must pass in a valid interface to MakeDelegateAdapter!"));
	return Private::TMAConverter<Callback>().Construct(Interface.ToSharedRef(), InDelegate, MoveTemp(InCallback));
}

template<typename ComponentType, typename DelegateType, typename Callback>
auto MakeMulticastAdapter(ComponentType* Interface, DelegateType& InDelegate, Callback&& InCallback)
{
	return Private::TMAConverter<Callback>().Construct(Interface->AsShared(), InDelegate, MoveTemp(InCallback));
}

/* UE::Online */ }
