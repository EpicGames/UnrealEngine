// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "OnlineSubsystemTypes.h"
#include "Misc/TransactionallySafeRWLock.h"

namespace UE::Online {

/**
 * A net id registry suitable for use with OSS FUniqueNetIds
 */
template<typename IdType>
class TOnlineUniqueNetIdRegistry : public IOnlineIdRegistry<IdType>
{
public:
	TOnlineUniqueNetIdRegistry(EOnlineServices InOnlineServicesType)
		: OnlineServicesType(InOnlineServicesType)
	{
	}

	virtual ~TOnlineUniqueNetIdRegistry() = default;

	TOnlineId<IdType> FindOrAddHandle(const FUniqueNetIdRef& IdValue)
	{
		TOnlineId<IdType> OnlineId;
		if (!ensure(IdValue->IsValid()))
		{
			return OnlineId;
		}

		// Take a read lock and check if we already have a handle
		{
			UE::TReadScopeLock ReadLock(Lock);
			if (const uint32* FoundHandle = IdValueToHandleMap.Find(IdValue))
			{
				OnlineId = TOnlineId<IdType>(OnlineServicesType, *FoundHandle);
			}
		}

		if (!OnlineId.IsValid())
		{
			// Take a write lock, check again if we already have a handle, or insert a new element.
			UE::TWriteScopeLock WriteLock(Lock);
			if (const uint32* FoundHandle = IdValueToHandleMap.Find(IdValue))
			{
				OnlineId = TOnlineId<IdType>(OnlineServicesType, *FoundHandle);
			}

			if (!OnlineId.IsValid())
			{
				IdValues.Emplace(IdValue);
				OnlineId = TOnlineId<IdType>(OnlineServicesType, IdValues.Num());
				IdValueToHandleMap.Emplace(IdValue, OnlineId.GetHandle());
			}
		}

		return OnlineId;
	}

	// Returns a copy as it's not thread safe to return a pointer/ref to an element of an array that can be relocated by another thread.
	FUniqueNetIdPtr GetIdValue(const TOnlineId<IdType> OnlineId) const
	{
		if (OnlineId.GetOnlineServicesType() == OnlineServicesType && OnlineId.IsValid())
		{
			UE::TReadScopeLock ReadLock(Lock);
			if (IdValues.IsValidIndex(OnlineId.GetHandle() - 1))
			{
				return IdValues[OnlineId.GetHandle() - 1];
			}
		}
		return FUniqueNetIdPtr();
	}

	FUniqueNetIdRef GetIdValueChecked(const TOnlineId<IdType> OnlineId) const
	{
		return GetIdValue(OnlineId).ToSharedRef();
	}

	bool IsHandleExpired(const FOnlineSessionId& InSessionId) const
	{
		return GetIdValue(InSessionId).IsValid();
	}

	// Begin IOnlineAccountIdRegistry
	virtual FString ToString(const TOnlineId<IdType>& OnlineId) const override
	{
		FUniqueNetIdPtr IdValue = GetIdValue(OnlineId);
		return IdValue ? IdValue->ToString() : TEXT("invalid_id");
	}

	virtual FString ToLogString(const TOnlineId<IdType>& OnlineId) const override
	{
		FUniqueNetIdPtr IdValue = GetIdValue(OnlineId);
		return IdValue ? IdValue->ToDebugString() : TEXT("invalid_id");
	}

	virtual TArray<uint8> ToReplicationData(const TOnlineId<IdType>& OnlineId) const override
	{
		// add a 0. empty array fails a check in FUniqueNetIdRepl::MakeReplicationDataV2
		return TArray<uint8>() = { 0 };
	}

	virtual TOnlineId<IdType> FromReplicationData(const TArray<uint8>& OnlineId) override
	{
		return TOnlineId<IdType>();
	}

	virtual TOnlineId<IdType> FromStringData(const FString& StringData) override
	{
		return TOnlineId<IdType>();
	}

	// End IOnlineAccountIdRegistry

private:
	mutable FTransactionallySafeRWLock Lock;

	EOnlineServices OnlineServicesType;
	TArray<FUniqueNetIdRef> IdValues;
	TUniqueNetIdMap<uint32> IdValueToHandleMap;
};

using FOnlineAccountIdRegistryOSSAdapter = TOnlineUniqueNetIdRegistry<OnlineIdHandleTags::FAccount>;

/* UE::Online */ }
