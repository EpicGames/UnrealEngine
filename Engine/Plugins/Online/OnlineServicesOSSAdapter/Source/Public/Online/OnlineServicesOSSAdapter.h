// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineServicesCommon.h"

#include "OnlineIdOSSAdapter.h"

#define UE_API ONLINESERVICESOSSADAPTER_API

class IOnlineSubsystem;

namespace UE::Online {

class FOnlineServicesOSSAdapter : public FOnlineServicesCommon
{
public:
	using Super = FOnlineServicesCommon;

	UE_API FOnlineServicesOSSAdapter(EOnlineServices InServicesType, const FString& InServiceConfigName, FName InInstanceName, IOnlineSubsystem* InSubsystem);

	UE_API virtual void RegisterComponents() override;
	UE_API virtual void Initialize() override;
	virtual EOnlineServices GetServicesProvider() const override { return ServicesType; }
	UE_API virtual TOnlineResult<FGetResolvedConnectString> GetResolvedConnectString(FGetResolvedConnectString::Params&& Params) override;

	IOnlineSubsystem& GetSubsystem() const { return *Subsystem; }
	FOnlineAccountIdRegistryOSSAdapter& GetAccountIdRegistry() const { return *AccountIdRegistry; }
	FOnlineAccountIdRegistryOSSAdapter& GetAccountIdRegistry() { return *AccountIdRegistry; }

protected:
	EOnlineServices ServicesType;
	IOnlineSubsystem* Subsystem;
	FOnlineAccountIdRegistryOSSAdapter* AccountIdRegistry;
};

/* UE::Online */ }

#undef UE_API
