// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"
#include "Modules/ModuleInterface.h"

namespace UE::Online {
	
	class FOnlineServicesOSSAdapter;

	class FOnlineServicesOSSAdapterModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		
		/**
		 * Get list of all instantiated OnlineServicesAdapter
		 * 
		 * @param OutOSSAdapters Array of online services adapters to fill
		 */
		ONLINESERVICESOSSADAPTER_API void GetAllAdapters(TArray<TSharedPtr<FOnlineServicesOSSAdapter>>& OutOSSAdapters);
	
		/**
		 * Register a new OnlineServicesAdapter to keep track of it
		 * 
		 * @param OnlineServicesOSSAdapter Online service adapter to track
		 */
		ONLINESERVICESOSSADAPTER_API void RegisterOSSAdapter(const TSharedPtr<FOnlineServicesOSSAdapter> OnlineServicesOSSAdapter);
		
	protected:
		void OnSubsystemPreReload(IOnlineSubsystem* OnlineSubsystem);
	
		FDelegateHandle OnSubsystemPreReloadHandle;
		TArray<TWeakPtr<FOnlineServicesOSSAdapter>> OSSAdapters;
		TArray<struct FOSSAdapterService> CachedServices;
	};
	
	/*UE::Online*/ }
