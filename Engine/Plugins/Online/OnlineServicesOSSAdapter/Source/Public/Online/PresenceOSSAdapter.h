// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/PresenceCommon.h"
#include "AuthOSSAdapter.h"
#include "OnlineSubsystemTypes.h"
#include "Interfaces/OnlinePresenceInterface.h"

class IOnlinePresence;
using IOnlinePresencePtr = TSharedPtr<IOnlinePresence>;

namespace UE::Online {

class FPresenceOSSAdapter : public FPresenceCommon
{
public:
	using Super = FPresenceCommon;

	using FPresenceCommon::FPresenceCommon;

	// IOnlineComponent
	virtual void PostInitialize() override;
	virtual void PreShutdown() override;

	// IPresence
	virtual TOnlineAsyncOpHandle<FQueryPresence> QueryPresence(FQueryPresence::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FBatchQueryPresence> BatchQueryPresence(FBatchQueryPresence::Params&& Params) override;
	virtual TOnlineResult<FGetCachedPresence> GetCachedPresence(FGetCachedPresence::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FUpdatePresence> UpdatePresence(FUpdatePresence::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FPartialUpdatePresence> PartialUpdatePresence(FPartialUpdatePresence::Params&& Params) override;

	FUniqueNetIdRef GetUniqueNetId(FAccountId AccountId) const;
	int32 GetLocalUserNum(FAccountId AccountId) const;

	ONLINESERVICESOSSADAPTER_API static EUserPresenceStatus StatusV1toV2(EOnlinePresenceState::Type Status);
	ONLINESERVICESOSSADAPTER_API static EOnlinePresenceState::Type StatusV2toV1(EUserPresenceStatus Status);

protected:
	const FAuthOSSAdapter* Auth;
	IOnlinePresencePtr PresenceInt;

	TSharedRef<FUserPresence> PresenceV1toV2(FAccountId Accountid, FOnlineUserPresence& Presence);
	TSharedRef<FOnlineUserPresence> PresenceV2toV1(FUserPresence& Presence);
};

/* UE::Online */ }
