// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/PrivilegesCommon.h"

class IOnlineIdentity;
using IOnlineIdentityPtr = TSharedPtr<IOnlineIdentity>;

namespace UE::Online {

class FPrivilegesOSSAdapter : public FPrivilegesCommon
{
public:
	using Super = FPrivilegesCommon;

	using FPrivilegesCommon::FPrivilegesCommon;

	// TOnlineComponent
	virtual void PostInitialize() override;
	virtual void PreShutdown() override;

	// IPermission
	virtual TOnlineAsyncOpHandle<FQueryUserPrivilege> QueryUserPrivilege(FQueryUserPrivilege::Params&& Params) override;

protected:
	IOnlineIdentityPtr GetIdentityInterface() const;
};

/* UE::Online */ }
