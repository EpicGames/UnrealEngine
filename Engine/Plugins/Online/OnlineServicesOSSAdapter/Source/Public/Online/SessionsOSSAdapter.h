// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SessionsCommon.h"

#include "OnlineSubsystemTypes.h"
#include "OnlineSessionSettings.h"

#include "Online/OnlineIdOSSAdapter.h"

using IOnlineSessionPtr = TSharedPtr<class IOnlineSession>;
using IOnlineIdentityPtr = TSharedPtr<class IOnlineIdentity>;

namespace UE::Online {

class FOnlineServicesOSSAdapter;

/** Registry of V1 session ids in FUniqueNetIdRef format, indexed by FOnlineSessionId */
using FOnlineSessionIdRegistryOSSAdapter = TOnlineUniqueNetIdRegistry<OnlineIdHandleTags::FSession>;

using FOnlineSessionInviteIdRegistryOSSAdapter = FOnlineSessionInviteIdStringRegistry;

static FName OSS_ADAPTER_SESSIONS_ALLOW_SANCTIONED_PLAYERS = TEXT("OSS_ADAPTER_SESSIONS_ALLOW_SANCTIONED_PLAYERS");
static FName OSS_ADAPTER_SESSIONS_ALLOW_UNREGISTERED_PLAYERS = TEXT("OSS_ADAPTER_SESSIONS_ALLOW_UNREGISTERED_PLAYERS");
static FName OSS_ADAPTER_SESSIONS_USE_LOBBIES_IF_AVAILABLE = TEXT("OSS_ADAPTER_SESSIONS_USE_LOBBIES_IF_AVAILABLE");
static FName OSS_ADAPTER_SESSIONS_USE_LOBBIES_VOICE_CHAT_IF_AVAILABLE = TEXT("OSS_ADAPTER_SESSIONS_USE_LOBBIES_VOICE_CHAT_IF_AVAILABLE");
static FName OSS_ADAPTER_SESSIONS_USES_STATS = TEXT("OSS_ADAPTER_SESSIONS_USES_STATS");
static FName OSS_ADAPTER_SESSIONS_SCHEMA_NAME = TEXT("OSS_ADAPTER_SESSIONS_SCHEMA_NAME");
static FName OSS_ADAPTER_SESSIONS_BUILD_UNIQUE_ID = TEXT("OSS_ADAPTER_SESSIONS_BUILD_UNIQUE_ID");
static FName OSS_ADAPTER_SESSIONS_PING_IN_MS = TEXT("OSS_ADAPTER_SESSIONS_PING_IN_MS");

static FName OSS_ADAPTER_SESSION_SEARCH_PING_BUCKET_SIZE = TEXT("OSS_ADAPTER_SESSION_SEARCH_PING_BUCKET_SIZE");
static FName OSS_ADAPTER_SESSION_SEARCH_PLATFORM_HASH = TEXT("OSS_ADAPTER_SESSION_SEARCH_PLATFORM_HASH");

class FSessionOSSAdapter : public FSessionCommon
{
public:
	FSessionOSSAdapter() = default;
	FSessionOSSAdapter(const FSessionOSSAdapter& InSession) = default;
	FSessionOSSAdapter(const FOnlineSession& InSession, const FOnlineSessionId& InSessionId);

	static const FSessionOSSAdapter& Cast(const ISession& InSession);

	const FOnlineSession& GetV1Session() const;

	virtual void DumpState() const override;

private:
	FOnlineSession V1Session;
};

class FSessionsOSSAdapter : public FSessionsCommon
{
public:
	using Super = FSessionsCommon;

	using FSessionsCommon::FSessionsCommon;

	FSessionsOSSAdapter(FOnlineServicesOSSAdapter& InOwningSubsystem);
	virtual ~FSessionsOSSAdapter() = default;

	// IOnlineComponent
	virtual void Initialize() override;
	virtual void Shutdown() override;

	// FSessionsCommon
	virtual TFuture<TOnlineResult<FCreateSession>> CreateSessionImpl(const FCreateSession::Params& Params) override;
	virtual TFuture<TOnlineResult<FUpdateSessionSettings>> UpdateSessionSettingsImpl(const FUpdateSessionSettings::Params& Params) override;
	virtual TFuture<TOnlineResult<FLeaveSession>> LeaveSessionImpl(const FLeaveSession::Params& Params) override;
	virtual TFuture<TOnlineResult<FFindSessions>> FindSessionsImpl(const FFindSessions::Params& Params) override;
	virtual TFuture<TOnlineResult<FStartMatchmaking>> StartMatchmakingImpl(const FStartMatchmaking::Params& Params) override;
	virtual TFuture<TOnlineResult<FJoinSession>> JoinSessionImpl(const FJoinSession::Params& Params) override;
	virtual TFuture<TOnlineResult<FAddSessionMember>> AddSessionMemberImpl(const FAddSessionMember::Params& Params) override;
	virtual TFuture<TOnlineResult<FRemoveSessionMember>> RemoveSessionMemberImpl(const FRemoveSessionMember::Params& Params) override;
	virtual TFuture<TOnlineResult<FSendSessionInvite>> SendSessionInviteImpl(const FSendSessionInvite::Params& Params) override;

	TOnlineResult<FGetResolvedConnectString> GetResolvedConnectString(const FGetResolvedConnectString::Params& Params);

protected:
	virtual TOptional<FOnlineError> CheckState(const FFindSessions::Params& Params) const override;
	virtual TOptional<FOnlineError> CheckState(const FJoinSession::Params& Params) const override;
	virtual TOptional<FOnlineError> CheckState(const FSendSessionInvite::Params& Params) const override;

private:
	/** Updates a local session's state by retrieving the equivalent from the OSS */
	bool UpdateV2Session(const FName& SessionName);

	/** Builds a V1 Session Settings data type from V2 session creation parameters */
	FOnlineSessionSettings BuildV1SettingsForCreate(const FCreateSession::Params& Params) const;
	/** Builds a V1 Session Settings data type from V2 session data */
	FOnlineSessionSettings BuildV1SettingsForUpdate(const FAccountId& LocalAccountId, const TSharedRef<ISession>& Session, const FSessionSettings& SessionSettings) const;
	/** Writes V1 Session Settings information into the passed V2 equivalent */
	void WriteV2SessionSettingsFromV1Session(const FOnlineSession* InSession, TSharedRef<FSessionOSSAdapter>& OutSession) const;
	/** Writes V1 Session Settings information contained in Named session type into the passed V2 equivalent */
	void WriteV2SessionSettingsFromV1NamedSession(const FNamedOnlineSession* InSession, TSharedRef<FSessionOSSAdapter>& OutSession) const;
	/** Builds a V1 Session data type from a V2 equivalent */
	FOnlineSession BuildV1Session(const TSharedRef<const ISession> InSession) const;
	/** Builds a V2 Session data type from a V1 equivalent */
	TSharedRef<FSessionCommon> BuildV2Session(const FNamedOnlineSession* InSession) const;
	TSharedRef<FSessionCommon> BuildV2Session(const FOnlineSession* InSession) const;
	/** Builds a V2 Session Search Results array from the passed V1 equivalent types */
	TArray<TSharedRef<FSessionCommon>> BuildV2SessionSearchResults(const TArray<FOnlineSessionSearchResult>& SessionSearchResults) const;

	/** Helper to get access to the account id and session id for V1 named sessions */
	bool GetAccountIdAndSessionIdFromNamedSession(const FName& SessionName, FAccountId& AccountId, FOnlineSessionId& OnlineSessionId) const;
	/** Helper to get the V2 session id from a v1 named session */
	bool GetV2SessionIdFromV1NamedSession(const FName& SessionName, FOnlineSessionId& OnlineSessionId) const;
	/** Helper to get the V2 session creator account id from a v1 named session */
	bool GetV2SessionCreatorFromV1NamedSession(const FName& SessionName, FAccountId& AccountId) const;

	FOnlineSessionIdRegistryOSSAdapter& GetSessionIdRegistry() const;
	FOnlineSessionInviteIdRegistryOSSAdapter& GetSessionInviteIdRegistry() const;

protected:
	IOnlineSessionPtr SessionsInterface;
	IOnlineIdentityPtr IdentityInterface;

	/** Cache of session search information to be used by ongoing session searches, indexed per user. Entries are removed upon search completion. */
	TMap<FAccountId, TSharedRef<FOnlineSessionSearch>> PendingV1SessionSearchesPerUser;
};

/* UE::Online */ }
