// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SocialCommon.h"
#include "AuthOSSAdapter.h"

class IOnlineFriends;
using IOnlineFriendsPtr = TSharedPtr<IOnlineFriends>;

namespace UE::Online {

class FSocialOSSAdapter : public FSocialCommon
{
public:
	using Super = FSocialCommon;

	using FSocialCommon::FSocialCommon;

	// IOnlineComponent
	virtual void PostInitialize() override;
	virtual void PreShutdown() override;

	// ISocial
	virtual TOnlineAsyncOpHandle<FQueryFriends> QueryFriends(FQueryFriends::Params&& Params) override;
	virtual TOnlineResult<FGetFriends> GetFriends(FGetFriends::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FSendFriendInvite> SendFriendInvite(FSendFriendInvite::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FAcceptFriendInvite> AcceptFriendInvite(FAcceptFriendInvite::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FRejectFriendInvite> RejectFriendInvite(FRejectFriendInvite::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FQueryBlockedUsers> QueryBlockedUsers(FQueryBlockedUsers::Params&& Params) override;
	virtual TOnlineResult<FGetBlockedUsers> GetBlockedUsers(FGetBlockedUsers::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FBlockUser> BlockUser(FBlockUser::Params&& Params) override;

protected:
	FAccountId GetAccountId(int32 LocalUserNum);
	void OnFriendsChanged();
	void OnBlockListChanged(int32 LocalUserNum, const FString& ListName);

	FDelegateHandle OnFriendsChangedDelegateHandles[MAX_LOCAL_PLAYERS];
	FDelegateHandle OnBlockListChangedDelegateHandles[MAX_LOCAL_PLAYERS];

	TMap<FAccountId, TMap<FAccountId, ERelationship>> Friends;
	TMap<FAccountId, TSet<FAccountId>> BlockedUsers;

	const FAuthOSSAdapter* Auth = nullptr;
	IOnlineFriendsPtr FriendsInt;
};

/* UE::Onlinm */  }