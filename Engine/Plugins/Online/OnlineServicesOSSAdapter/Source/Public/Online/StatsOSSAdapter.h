// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/StatsCommon.h"
#include "AuthOSSAdapter.h"
#include "OnlineSubsystemTypes.h"
#include "Interfaces/OnlineStatsInterface.h"

class IOnlineStats;
using IOnlineStatsPtr = TSharedPtr<IOnlineStats>;

namespace UE::Online {

class FStatsOSSAdapter : public FStatsCommon
{
public:
	using Super = FStatsCommon;

	using FStatsCommon::FStatsCommon;

	// IOnlineComponent
	virtual void PostInitialize() override;

	// IStats
	virtual TOnlineAsyncOpHandle<FUpdateStats> UpdateStats(FUpdateStats::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FQueryStats> QueryStats(FQueryStats::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FBatchQueryStats> BatchQueryStats(FBatchQueryStats::Params&& Params) override;
#if !UE_BUILD_SHIPPING
	virtual TOnlineAsyncOpHandle<FResetStats> ResetStats(FResetStats::Params&& Params) override;
#endif // !UE_BUILD_SHIPPING

private:
	FOnlineError ConvertUpdateUsersStatsV2toV1(const TArray<FUserStats>& UpdateUsersStats, TArray<FOnlineStatsUserUpdatedStats>& OutUpdateUserStatsV1);
	void ConvertStatValueV2ToStatUpdateV1(const FString& StatName, const FStatValue& StatValue, FOnlineStatUpdate& OutOnlineStatUpdate);
	void ConvertStatValueV1ToStatValueV2(const FOnlineStatValue& OnlineStatValue, FStatValue& OutStatValue);

	const FAuthOSSAdapter* Auth;
	IOnlineStatsPtr StatsInterface;
};

/* UE::Online */ }
