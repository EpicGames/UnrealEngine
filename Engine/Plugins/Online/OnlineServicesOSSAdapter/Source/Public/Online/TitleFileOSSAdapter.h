// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/TitleFileCommon.h"

#include "OnlineSubsystemTypes.h"

using IOnlineTitleFilePtr = TSharedPtr<class IOnlineTitleFile>;

namespace UE::Online {

class FTitleFileOSSAdapter : public FTitleFileCommon
{
public:
	using Super = FTitleFileCommon;

	using FTitleFileCommon::FTitleFileCommon;

	// IOnlineComponent
	virtual void PostInitialize() override;

	// ITitleFile
	virtual TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> EnumerateFiles(FTitleFileEnumerateFiles::Params&& Params) override;
	virtual TOnlineResult<FTitleFileGetEnumeratedFiles> GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FTitleFileReadFile> ReadFile(FTitleFileReadFile::Params&& Params) override;

protected:
	IOnlineTitleFilePtr TitleFileInterface = nullptr;

	TOptional<TArray<FString>> EnumeratedFiles;
};

/* UE::Online */ }
