// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/UserFileCommon.h"

#include "OnlineSubsystemTypes.h"

using IOnlineUserCloudPtr = TSharedPtr<class IOnlineUserCloud>;

namespace UE::Online {

struct FUserFileOSSAdapterConfig
{
	bool bCompressBeforeUpload = false;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FUserFileOSSAdapterConfig)
	ONLINE_STRUCT_FIELD(FUserFileOSSAdapterConfig, bCompressBeforeUpload)
END_ONLINE_STRUCT_META()
/* Meta */ }

class FUserFileOSSAdapter : public FUserFileCommon
{
public:
	using Super = FUserFileCommon;

	using FUserFileCommon::FUserFileCommon;

	// IOnlineComponent
	virtual void PostInitialize() override;
	virtual void UpdateConfig() override;

	// IUserFile
	virtual TOnlineAsyncOpHandle<FUserFileEnumerateFiles> EnumerateFiles(FUserFileEnumerateFiles::Params&& Params) override;
	virtual TOnlineResult<FUserFileGetEnumeratedFiles> GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FUserFileReadFile> ReadFile(FUserFileReadFile::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FUserFileWriteFile> WriteFile(FUserFileWriteFile::Params&& Params) override;
	//virtual TOnlineAsyncOpHandle<FUserFileCopyFile> CopyFile(FUserFileCopyFile::Params&& Params) override; // Unimplemented
	virtual TOnlineAsyncOpHandle<FUserFileDeleteFile> DeleteFile(FUserFileDeleteFile::Params&& Params) override;

protected:
	FUserFileOSSAdapterConfig Config;
	IOnlineUserCloudPtr UserCloudInterface = nullptr;
	TMap<FAccountId, TArray<FString>> EnumeratedFiles;
};

/* UE::Online */ }
