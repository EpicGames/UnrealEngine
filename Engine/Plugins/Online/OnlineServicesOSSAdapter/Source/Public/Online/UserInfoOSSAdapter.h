// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/UserInfoCommon.h"

#include "OnlineSubsystemTypes.h"

using IOnlineUserPtr = TSharedPtr<class IOnlineUser>;

namespace UE::Online {

class FUserInfoOSSAdapter : public FUserInfoCommon
{
public:
	using Super = FUserInfoCommon;

	using FUserInfoCommon::FUserInfoCommon;

	// IOnlineComponent
	virtual void Initialize() override;

	// IUserInfo
	virtual TOnlineAsyncOpHandle<FQueryUserInfo> QueryUserInfo(FQueryUserInfo::Params&& Params) override;
	virtual TOnlineResult<FGetUserInfo> GetUserInfo(FGetUserInfo::Params&& Params) override;

protected:
	IOnlineUserPtr UserInterface = nullptr;
};

/* UE::Online */ }
